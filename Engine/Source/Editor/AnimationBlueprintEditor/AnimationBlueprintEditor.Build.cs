// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class AnimationBlueprintEditor : ModuleRules
{
	public AnimationBlueprintEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
                "SkeletonEditor",
				"MessageLog"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "AppFramework",
				"Core", 
				"CoreUObject", 
				"Slate", 
				"SlateCore",
                "EditorStyle",
				"Engine", 
				"UnrealEd", 
				"GraphEditor", 
                "InputCore",
				"KismetWidgets",
				"AnimGraph",
				"AnimGraphRuntime",
				"PropertyEditor",
				"EditorWidgets",
                "BlueprintGraph",
                "RHI",
                "KismetCompiler",
				"ToolMenus",
				"AnimGraphRuntime",
            }
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"ContentBrowser",
				"Documentation",
				"MainFrame",
				"DesktopPlatform",
                "SkeletonEditor",
                "AssetTools",
                "AnimationEditor",
            }
		);

        PublicDependencyModuleNames.AddRange(
            new string[] {
                "Kismet",
                "Persona",
				"EditorFramework"
            }
        );
    }
}
