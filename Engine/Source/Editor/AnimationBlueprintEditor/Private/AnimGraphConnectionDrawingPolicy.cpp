// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimGraphConnectionDrawingPolicy.h"

#include "AnimGraphAttributes.h"
#include "AnimGraphNode_Base.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "AnimationGraphSchema.h"
#include "AnimationPins/SGraphPinPose.h"
#include "ConnectionDrawingPolicy.h"
#include "Containers/Array.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Set.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/Blueprint.h"
#include "K2Node_Knot.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Math/Color.h"
#include "Math/Range.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "SGraphPin.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/UObjectGlobals.h"

class FArrangedWidget;
class FSlateRect;
class SWidget;

/////////////////////////////////////////////////////
// FAnimGraphConnectionDrawingPolicy

FAnimGraphConnectionDrawingPolicy::FAnimGraphConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj)
	: FKismetConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements, InGraphObj)
{
}

bool FAnimGraphConnectionDrawingPolicy::TreatWireAsExecutionPin(UEdGraphPin* InputPin, UEdGraphPin* OutputPin) const
{
	const UAnimationGraphSchema* Schema = GetDefault<UAnimationGraphSchema>();

	return (InputPin != NULL) && (Schema->IsPosePin(OutputPin->PinType));
}

void FAnimGraphConnectionDrawingPolicy::BuildExecutionRoadmap()
{
	if(UAnimBlueprint* TargetBP = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForGraph(GraphObj)))
	{
		UAnimBlueprintGeneratedClass* AnimBlueprintClass = (UAnimBlueprintGeneratedClass*)(*(TargetBP->GeneratedClass));

		if (TargetBP->GetObjectBeingDebugged() == NULL)
		{
			return;
		}
		
		FAnimBlueprintDebugData& DebugInfo = AnimBlueprintClass->GetAnimBlueprintDebugData();
		for (auto VisitIt = DebugInfo.UpdatedNodesThisFrame.CreateIterator(); VisitIt; ++VisitIt)
		{
			const FAnimBlueprintDebugData::FNodeVisit& VisitRecord = *VisitIt;

			const int32 NumAnimNodeProperties = AnimBlueprintClass->GetAnimNodeProperties().Num();
			if ((VisitRecord.SourceID >= 0) && (VisitRecord.SourceID < NumAnimNodeProperties) && (VisitRecord.TargetID >= 0) && (VisitRecord.TargetID < NumAnimNodeProperties))
			{
				const int32 ReverseSourceID = NumAnimNodeProperties - 1 - VisitRecord.SourceID;
				const int32 ReverseTargetID = NumAnimNodeProperties - 1 - VisitRecord.TargetID;
				
				if (const UAnimGraphNode_Base* SourceNode = Cast<const UAnimGraphNode_Base>(DebugInfo.NodePropertyIndexToNodeMap.FindRef(ReverseSourceID)))
				{
					if (const UAnimGraphNode_Base* TargetNode = Cast<const UAnimGraphNode_Base>(DebugInfo.NodePropertyIndexToNodeMap.FindRef(ReverseTargetID)))
					{
						UEdGraphPin* PoseNet = NULL;

						UAnimationGraphSchema const* AnimSchema = GetDefault<UAnimationGraphSchema>();
						for (int32 PinIndex = 0; PinIndex < TargetNode->Pins.Num(); ++PinIndex)
						{
							UEdGraphPin* Pin = TargetNode->Pins[PinIndex];
							check(Pin);
							if (AnimSchema->IsPosePin(Pin->PinType) && (Pin->Direction == EGPD_Output))
							{
								PoseNet = Pin;
								break;
							}
						}

						if (PoseNet != NULL)
						{
							//@TODO: Extend the rendering code to allow using the recorded blend weight instead of faked exec times
							FExecPairingMap& Predecessors = PredecessorPins.FindOrAdd((UEdGraphNode*)SourceNode);
							FTimePair& Timings = Predecessors.FindOrAdd(PoseNet);
							Timings.PredExecTime = 0.0;
							Timings.ThisExecTime = FMath::Clamp(VisitRecord.Weight, 0.f, 1.f);
						}
					}
				}
			}
		}
	}
}

void FAnimGraphConnectionDrawingPolicy::BuildPinToPinWidgetMap(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries)
{
	FKismetConnectionDrawingPolicy::BuildPinToPinWidgetMap(InPinGeometries);

	// Cache additional attributes
	PinAttributes.Reset();
	bool bFoundPanelZoom = false;
	PanelZoom = 1.0f;

	for(const TPair<UEdGraphPin*, TSharedPtr<SGraphPin>>& PinWidgetPair : PinToPinWidgetMap)
	{
		if(PinWidgetPair.Key->Direction == EGPD_Output && UAnimationGraphSchema::IsPosePin(PinWidgetPair.Key->PinType) && PinWidgetPair.Key->GetOwningNode()->IsA<UAnimGraphNode_Base>())
		{
			// Pose pins are assumed to be SGraphPinPose widgets here
			check(PinWidgetPair.Value->GetType() == TEXT("SGraphPinPose"));
			TSharedPtr<SGraphPinPose> PosePin = StaticCastSharedPtr<SGraphPinPose>(PinWidgetPair.Value);

			PinAttributes.Add(PinWidgetPair.Key, PosePin->GetAttributeInfo());

			if(!bFoundPanelZoom)
			{
				PanelZoom = PosePin->GetZoomAmount();
				bFoundPanelZoom = true;
			}
		}
	}
}

void FAnimGraphConnectionDrawingPolicy::DetermineStyleOfExecWire(float& Thickness, FLinearColor& WireColor, bool& bDrawBubbles, const FTimePair& Times)
{
	// It's a followed link, make it strong and yellowish but fading over time
	const double BlendWeight = Times.ThisExecTime;

	const float HeavyBlendThickness = AttackWireThickness;
	const float LightBlendThickness = SustainWireThickness;

	Thickness = FMath::Lerp<float>(LightBlendThickness, HeavyBlendThickness, BlendWeight);
	WireColor = WireColor * static_cast<float>(BlendWeight * 0.5 + 0.5);//FMath::Lerp<FLinearColor>(SustainColor, AttackColor, BlendWeight);

	bDrawBubbles = true;
}

void FAnimGraphConnectionDrawingPolicy::DrawConnection(int32 LayerId, const FVector2f& Start, const FVector2f& End, const FConnectionParams& Params) 
{
	bool bCompositeWire = false;

	// Pose pins display attribute links
	if(Params.AssociatedPin1 && Params.AssociatedPin2)
	{
		if(UAnimationGraphSchema::IsPosePin(Params.AssociatedPin1->PinType))
		{
			// If either pin connects to a re-route (knot) node, traverse the link until a relevant pin is found
			UEdGraphPin* UsePin1 = Params.AssociatedPin1;
			if (Params.AssociatedPin1->GetOwningNode()->IsA<UK2Node_Knot>())
			{
				UsePin1 = FBlueprintEditorUtils::FindFirstCompilerRelevantLinkedPin(Params.AssociatedPin1);
			}

			UEdGraphPin* UsePin2 = Params.AssociatedPin2;
			if (Params.AssociatedPin2->GetOwningNode()->IsA<UK2Node_Knot>())
			{
				UsePin2 = FBlueprintEditorUtils::FindFirstCompilerRelevantLinkedPin(Params.AssociatedPin2);
			}

			if (UsePin1 && UsePin2)
			{
				UAnimGraphNode_Base* Node1 = Cast<UAnimGraphNode_Base>(UsePin1->GetOwningNode());
				UAnimGraphNode_Base* Node2 = Cast<UAnimGraphNode_Base>(UsePin2->GetOwningNode());
				if (Node1 && Node2)
				{
					const TArrayView<const SGraphPinPose::FAttributeInfo>& AdditionalAttributeInfo = PinAttributes.FindRef(UsePin1);

					if (AdditionalAttributeInfo.Num() > 0)
					{
						const float MaxAttributeWireThickness = 3.0f;
						const float MinAttributeWireThickness = 1.0f;
						const float MaxWireGap = 2.0f;
						const float MinWireGap = 0.5f;

						// 0.375f is the zoom level before the 'low LOD' cutoff
						const float ZoomLevelAlpha = FMath::GetMappedRangeValueClamped(TRange<float>(0.375f, 1.0f), TRange<float>(0.0f, 1.0f), PanelZoom);
						const float AttributeWireThickness = FMath::Lerp(MinAttributeWireThickness, MaxAttributeWireThickness, ZoomLevelAlpha);
						const float WireGap = FMath::Lerp(MinWireGap, MaxWireGap, ZoomLevelAlpha);

						const FVector2f& P0 = Start;
						const FVector2f& P1 = End;

						const FVector2f SplineTangent = ComputeSplineTangent(P0, P1);
						const FVector2f P0Tangent = (Params.StartDirection == EGPD_Output) ? SplineTangent : -SplineTangent;
						const FVector2f P1Tangent = (Params.EndDirection == EGPD_Input) ? SplineTangent : -SplineTangent;

						bCompositeWire = true;

						float TotalThickness = Params.WireThickness;

						static TArray<float> CachedWireThicknesses;
						check(CachedWireThicknesses.Num() == 0);	// Cant be called recursively or on multiple threads
						CachedWireThicknesses.SetNumZeroed(AdditionalAttributeInfo.Num());

						for (int32 AttributeInfoIndex = 0; AttributeInfoIndex < AdditionalAttributeInfo.Num(); ++AttributeInfoIndex)
						{
							const SGraphPinPose::FAttributeInfo& AttributeInfo = AdditionalAttributeInfo[AttributeInfoIndex];

							float WireThickness = 0.0f;
							switch (AttributeInfo.Blend)
							{
							case EAnimGraphAttributeBlend::Blendable:
							{
								if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForGraph(GraphObj)))
								{
									UAnimBlueprintGeneratedClass* AnimBlueprintClass = (UAnimBlueprintGeneratedClass*)(*(AnimBlueprint->GeneratedClass));
									int32 SourceNodeId = AnimBlueprintClass->GetNodeIndexFromGuid(Node1->NodeGuid);
									int32 TargetNodeId = AnimBlueprintClass->GetNodeIndexFromGuid(Node2->NodeGuid);
									if (SourceNodeId != INDEX_NONE && TargetNodeId != INDEX_NONE)
									{
										const TArray<FAnimBlueprintDebugData::FAttributeRecord>* LinkAttributes = AnimBlueprintClass->GetAnimBlueprintDebugData().NodeOutputAttributesThisFrame.Find(SourceNodeId);
										const bool bAttributeUsedInLink = LinkAttributes && LinkAttributes->ContainsByPredicate(
											[&AttributeInfo, TargetNodeId](const FAnimBlueprintDebugData::FAttributeRecord& InRecord)
											{
												return InRecord.Attribute == AttributeInfo.Attribute && InRecord.OtherNode == TargetNodeId;
											});


										WireThickness = bAttributeUsedInLink ? AttributeWireThickness : 0.0f;
									}
								}
								break;
							}
							case EAnimGraphAttributeBlend::NonBlendable:
								WireThickness = AttributeWireThickness;
								break;
							}

							CachedWireThicknesses[AttributeInfoIndex] = WireThickness;
							TotalThickness += WireThickness != 0.0f ? (WireThickness + WireGap) : 0.0f;
						}

						const float InitialOffset = TotalThickness * 0.5f;
						FVector2f SubWireP0 = P0;
						SubWireP0.Y += InitialOffset;
						FVector2f SubWireP1 = P1;
						SubWireP1.Y += InitialOffset;

						// Draw in reverse order to get pose wires appearing on top
						for (int32 AttributeInfoIndex = AdditionalAttributeInfo.Num() - 1; AttributeInfoIndex >= 0; --AttributeInfoIndex)
						{
							float Thickness = CachedWireThicknesses[AttributeInfoIndex];
							if (Thickness > 0.0f)
							{
								const SGraphPinPose::FAttributeInfo& AttributeInfo = AdditionalAttributeInfo[AttributeInfoIndex];
								FLinearColor Color = AttributeInfo.Color;

								if (HoveredPins.Num() > 0)
								{
									ApplyHoverDeemphasis(Params.AssociatedPin1, Params.AssociatedPin2, Thickness, Color);
								}

								SubWireP0.Y -= Thickness + WireGap;
								SubWireP1.Y -= Thickness + WireGap;

								// Draw the spline itself
								FSlateDrawElement::MakeDrawSpaceSpline(
									DrawElementsList,
									LayerId,
									SubWireP0, P0Tangent,
									SubWireP1, P1Tangent,
									Thickness,
									ESlateDrawEffect::None,
									Color
								);
							}
						}

						SubWireP0.Y -= Params.WireThickness + WireGap;
						SubWireP1.Y -= Params.WireThickness + WireGap;

						FKismetConnectionDrawingPolicy::DrawConnection(LayerId, SubWireP0, SubWireP1, Params);

						CachedWireThicknesses.Reset();
					}
				}
			}
		}
	}

	if(!bCompositeWire)
	{
		FKismetConnectionDrawingPolicy::DrawConnection(LayerId, Start, End, Params);
	}
}

void FAnimGraphConnectionDrawingPolicy::ApplyHoverDeemphasis(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, float& Thickness, FLinearColor& WireColor)
{
	// Remove the thickness increase on hover
	float OriginalThickness = Thickness;
	FKismetConnectionDrawingPolicy::ApplyHoverDeemphasis(OutputPin, InputPin, Thickness, WireColor);
	Thickness = OriginalThickness;
}