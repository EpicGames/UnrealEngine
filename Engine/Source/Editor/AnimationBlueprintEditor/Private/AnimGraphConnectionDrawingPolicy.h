// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationPins/SGraphPinPose.h"
#include "BlueprintConnectionDrawingPolicy.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Delegates/IDelegateInstance.h"
#include "HAL/Platform.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"

class FArrangedWidget;
class FSlateRect;
class FSlateWindowElementList;
class SWidget;
class UEdGraph;
class UEdGraphPin;
struct FConnectionParams;
struct FLinearColor;

/////////////////////////////////////////////////////
// FAnimGraphConnectionDrawingPolicy

// This class draws the connections for an UEdGraph with an animation schema
class FAnimGraphConnectionDrawingPolicy : public FKismetConnectionDrawingPolicy
{
public:
	// Constructor
	FAnimGraphConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj);

	// FKismetConnectionDrawingPolicy interface
	virtual bool TreatWireAsExecutionPin(UEdGraphPin* InputPin, UEdGraphPin* OutputPin) const override;
	virtual void BuildExecutionRoadmap() override;
	virtual void DetermineStyleOfExecWire(float& Thickness, FLinearColor& WireColor, bool& bDrawBubbles, const FTimePair& Times) override;
	virtual void DrawConnection(int32 LayerId, const FVector2f& Start, const FVector2f& End, const FConnectionParams& Params) override;
	virtual void BuildPinToPinWidgetMap(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries) override;
	virtual void ApplyHoverDeemphasis(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ float& Thickness, /*inout*/ FLinearColor& WireColor) override;
	// End of FKismetConnectionDrawingPolicy interface

private:
	// Map to cached attribute array on the node
	TMap<UEdGraphPin*, TArrayView<const SGraphPinPose::FAttributeInfo>> PinAttributes;
	
	// Handle to compilation delegate
	FDelegateHandle OnBlueprintCompiledHandle;

	// Zoom level of the current panel
	float PanelZoom;
};

