// Copyright Epic Games, Inc. All Rights Reserved.


#include "AnimationBlueprintEditor.h"

#include "Algo/Transform.h"
#include "AnimGraphCommands.h"
#include "AnimGraphNode_AimOffsetLookAt.h"
#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_BlendListByInt.h"
#include "AnimGraphNode_BlendSpaceEvaluator.h"
#include "AnimGraphNode_BlendSpaceGraph.h"
#include "AnimGraphNode_BlendSpacePlayer.h"
#include "AnimGraphNode_LayeredBoneBlend.h"
#include "AnimGraphNode_LinkedAnimGraphBase.h"
#include "AnimGraphNode_MultiWayBlend.h"
#include "AnimGraphNode_PoseBlendNode.h"
#include "AnimGraphNode_PoseByName.h"
#include "AnimGraphNode_RotationOffsetBlendSpace.h"
#include "AnimGraphNode_RotationOffsetBlendSpaceGraph.h"
#include "AnimGraphNode_SequenceEvaluator.h"
#include "AnimGraphNode_SequencePlayer.h"
#include "AnimNodes/AnimNode_AimOffsetLookAt.h"
#include "AnimNodes/AnimNode_BlendSpaceEvaluator.h"
#include "AnimNodes/AnimNode_BlendSpacePlayer.h"
#include "AnimNodes/AnimNode_PoseBlendNode.h"
#include "AnimNodes/AnimNode_PoseByName.h"
#include "AnimNodes/AnimNode_RotationOffsetBlendSpace.h"
#include "AnimNodes/AnimNode_SequenceEvaluator.h"
#include "AnimPreviewInstance.h"
#include "AnimStateEntryNode.h"
#include "AnimStateNodeBase.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimNode_SequencePlayer.h"
#include "Animation/AnimNotifies/AnimNotifyState.h"
#include "Animation/AnimNotifyQueue.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimTypes.h"
#include "Animation/AnimationAsset.h"
#include "Animation/BlendSpace.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimationBlueprintEditorMode.h"
#include "AnimationBlueprintEditorModule.h"
#include "AnimationBlueprintEditorSettings.h"
#include "AnimationBlueprintInterfaceEditorMode.h"
#include "AnimationEditorUtils.h"
#include "AnimationGraph.h"
#include "AnimationGraphSchema.h"
#include "AssetRegistry/AssetData.h"
#include "BlendSpaceDocumentTabFactory.h"
#include "BlendSpaceGraph.h"
#include "Blueprint/BlueprintExceptionInfo.h"
#include "Components/SkeletalMeshComponent.h"
#include "Containers/EnumAsByte.h"
#include "CoreGlobals.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphNode_Comment.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphSchema_K2_Actions.h"
#include "EdGraphUtilities.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "EditorFontGlyphs.h"
#include "EditorReimportHandler.h"
#include "Engine/Blueprint.h"
#include "Engine/PoseWatch.h"
#include "Engine/World.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "HAL/PlatformCrt.h"
#include "IAnimationBlueprintEditorModule.h"
#include "IAssetFamily.h"
#include "IDetailsView.h"
#include "IPersonaEditorModeManager.h"
#include "IPersonaPreviewScene.h"
#include "IPersonaToolkit.h"
#include "IPersonaViewport.h"
#include "ISkeletonEditorModule.h"
#include "ISkeletonTree.h"
#include "ISkeletonTreeItem.h"
#include "Input/Reply.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/DebuggerCommands.h"
#include "Kismet2/Kismet2NameValidators.h"
#include "Layout/Visibility.h"
#include "Logging/TokenizedMessage.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PersonaCommonCommands.h"
#include "PersonaDelegates.h"
#include "PersonaModule.h"
#include "PersonaToolMenuContext.h"
#include "PersonaUtils.h"
// Hide related nodes feature
#include "AnimationBlueprintToolMenuContext.h"
#include "AnimStateAliasNode.h"
#include "AnimStateNode.h"
#include "AnimStateTransitionNode.h"
#include "DetailLayoutBuilder.h"
#include "Preferences/AnimationBlueprintEditorOptions.h"
#include "PropertyEditorDelegates.h"
#include "SBlueprintEditorToolbar.h"
#include "SKismetInspector.h"
#include "SSingleObjectDetailsPanel.h"
#include "ScopedTransaction.h"
#include "Settings/AnimBlueprintSettings.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Subsystems/ImportSubsystem.h"
#include "TabPayload_BlendSpaceGraph.h"
#include "Templates/Casts.h"
#include "Templates/SubclassOf.h"
#include "ToolMenuContext.h"
#include "ToolMenus.h"
#include "Animation/AnimNode_StateMachine.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

class FEditorModeTools;
class FToolBarBuilder;
class IAnimationSequenceBrowser;
class SDockTab;
class SWidget;

#define LOCTEXT_NAMESPACE "AnimationBlueprintEditor"

const FName AnimationBlueprintEditorAppName(TEXT("AnimationBlueprintEditorApp"));

const FName FAnimationBlueprintEditorModes::AnimationBlueprintEditorMode("GraphName");	// For backwards compatibility we keep the old mode name here
const FName FAnimationBlueprintEditorModes::AnimationBlueprintInterfaceEditorMode("Interface");
const FName FAnimationBlueprintEditorModes::AnimationBlueprintTemplateEditorMode("Template");

namespace AnimationBlueprintEditorTabs
{
	const FName DetailsTab(TEXT("DetailsTab"));
	const FName SkeletonTreeTab(TEXT("SkeletonTreeView"));
	const FName ViewportTab(TEXT("Viewport"));
	const FName AdvancedPreviewTab(TEXT("AdvancedPreviewTab"));
	const FName AssetBrowserTab(TEXT("SequenceBrowser"));
	const FName AnimBlueprintPreviewEditorTab(TEXT("AnimBlueprintPreviewEditor"));
	const FName AssetOverridesTab(TEXT("AnimBlueprintParentPlayerEditor"));
	const FName SlotNamesTab(TEXT("SkeletonSlotNames"));
	const FName CurveNamesTab(TEXT("AnimCurveViewerTab"));
	const FName PoseWatchTab(TEXT("PoseWatchManager"));
	const FName FindReplaceTab(TEXT("FindReplaceTab"));
};

/////////////////////////////////////////////////////
// SortedContainerDifference

/** Algorithm to find the difference between two sorted sets of unique values - outputs two sets, all the elements that are in set A but not in set B and all the elements that are in set B but not in set A **/
template <typename TContainerType, typename TPredicate> void SortedContainerDifference(const TContainerType& LhsContainer, const TContainerType& RhsContainer, TContainerType& OutLhsDifference, TContainerType& OutRhsDifference, const TPredicate& SortPredicate)
{
	for (unsigned int LhsIndex = 0, RhsIndex = 0, LhsMax = LhsContainer.Num(), RhsMax = RhsContainer.Num(); (LhsIndex < LhsMax) || (RhsIndex < RhsMax); )
	{
		if ((LhsIndex < LhsMax) && (!(RhsIndex < RhsMax) || SortPredicate(LhsContainer[LhsIndex], RhsContainer[RhsIndex])))
		{
			OutRhsDifference.Add(LhsContainer[LhsIndex]);
			++LhsIndex;
		}
		else if ((RhsIndex < RhsMax) && (!(LhsIndex < LhsMax) || SortPredicate(RhsContainer[RhsIndex], LhsContainer[LhsIndex])))
		{
			OutLhsDifference.Add(RhsContainer[RhsIndex]);
			++RhsIndex;
		}
		else
		{
			++LhsIndex;
			++RhsIndex;
		}
	}
}

void FAnimationBlueprintEditor::NotifyAllNodesOnSelection(const bool bInIsSelected)
{
	FEditorModeTools& ModeTools = GetEditorModeManager();

	for (TWeakObjectPtr< class UAnimGraphNode_Base > CurrentAnimGraphNode : SelectedAnimGraphNodes)
	{
		UAnimGraphNode_Base* const CurrentAnimGraphNodePtr = CurrentAnimGraphNode.Get();
		FAnimNode_Base* const PreviewNode = FindAnimNode(CurrentAnimGraphNodePtr);
		
		// Note: Potentially passing a null PreviewNode ptr when bInIsSelected is false is required to de-select nodes that no longer exist.
		if (CurrentAnimGraphNodePtr && (!bInIsSelected || PreviewNode))
		{
			CurrentAnimGraphNodePtr->OnNodeSelected(bInIsSelected, ModeTools, PreviewNode);
		}
	}
}

void FAnimationBlueprintEditor::NotifyAllNodesOnPoseWatchChanged(const bool IsPoseWatchEnabled)
{
	UAnimBlueprint* const AnimBP = GetAnimBlueprint();

	if (AnimBP)
	{
		FEditorModeTools& ModeTools = GetEditorModeManager();

		for (const TObjectPtr<UPoseWatch>& CurrentPoseWatch : AnimBP->PoseWatches)
		{
			UAnimGraphNode_Base* const CurrentAnimGraphNodePtr = Cast<UAnimGraphNode_Base>(CurrentPoseWatch->Node.Get());
			FAnimNode_Base* const PreviewNode = FindAnimNode(CurrentAnimGraphNodePtr);

			// Note: Potentially passing a null PreviewNode ptr when IsPoseWatchEnabled is false is required to un-watch nodes that no longer exist.
			if (CurrentAnimGraphNodePtr && (!IsPoseWatchEnabled || PreviewNode))
			{
				CurrentAnimGraphNodePtr->OnPoseWatchChanged(IsPoseWatchEnabled, CurrentPoseWatch, ModeTools, PreviewNode);
			}
		}
	}
}

void FAnimationBlueprintEditor::ReleaseAllManagedNodes()
{
	NotifyAllNodesOnPoseWatchChanged(false);
	NotifyAllNodesOnSelection(false);
}

void FAnimationBlueprintEditor::AcquireAllManagedNodes()
{
	NotifyAllNodesOnPoseWatchChanged(true);
	NotifyAllNodesOnSelection(true);
}

/////////////////////////////////////////////////////
// SAnimBlueprintPreviewPropertyEditor

class SAnimBlueprintPreviewPropertyEditor : public SSingleObjectDetailsPanel
{
public:
	SLATE_BEGIN_ARGS(SAnimBlueprintPreviewPropertyEditor) {}
	SLATE_END_ARGS()

private:
	// Pointer back to owning Persona editor instance (the keeper of state)
	TWeakPtr<FAnimationBlueprintEditor> AnimationBlueprintEditorPtr;
public:
	void Construct(const FArguments& InArgs, TSharedPtr<FAnimationBlueprintEditor> InAnimationBlueprintEditor)
	{
		AnimationBlueprintEditorPtr = InAnimationBlueprintEditor;

		SSingleObjectDetailsPanel::Construct(SSingleObjectDetailsPanel::FArguments().HostCommandList(InAnimationBlueprintEditor->GetToolkitCommands()).HostTabManager(InAnimationBlueprintEditor->GetTabManager()), /*bAutomaticallyObserveViaGetObjectToObserve*/ true, /*bAllowSearch*/ true);

		PropertyView->SetIsPropertyEditingEnabledDelegate(FIsPropertyEditingEnabled::CreateStatic([] { return !GIntraFrameDebuggingGameThread; }));
	}

	// SSingleObjectDetailsPanel interface
	virtual UObject* GetObjectToObserve() const override
	{
		if (UDebugSkelMeshComponent* PreviewMeshComponent = AnimationBlueprintEditorPtr.Pin()->GetPersonaToolkit()->GetPreviewMeshComponent())
		{
			return PreviewMeshComponent->GetAnimInstance();
		}

		return nullptr;
	}

	virtual TSharedRef<SWidget> PopulateSlot(TSharedRef<SWidget> PropertyEditorWidget) override
	{
		return SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.f, 8.f, 0.f, 0.f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Persona.PreviewPropertiesWarning"))
				[
					SNew(STextBlock)
					.Text(LOCTEXT("AnimBlueprintEditPreviewText", "Changes to preview options are not saved in the asset."))
					.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
					.ShadowColorAndOpacity(FLinearColor::Black.CopyWithNewOpacity(0.3f))
					.ShadowOffset(FVector2D::UnitVector)
				]
			]
			+SVerticalBox::Slot()
			.FillHeight(1)
			[
				PropertyEditorWidget
			];
	}
	// End of SSingleObjectDetailsPanel interface
};

/////////////////////////////////////////////////////
// FAnimationBlueprintEditor

FAnimationBlueprintEditor::FAnimationBlueprintEditor()
	: PersonaMeshDetailLayout(nullptr)
	, DebuggedMeshComponent(nullptr)
{
	GEditor->OnBlueprintPreCompile().AddRaw(this, &FAnimationBlueprintEditor::OnBlueprintPreCompile);
	LastGraphPinType.ResetToDefaults();
	LastGraphPinType.PinCategory = UEdGraphSchema_K2::PC_Boolean;
}

FAnimationBlueprintEditor::~FAnimationBlueprintEditor()
{
	// Stop watching the settings
	UAnimationBlueprintEditorSettings* AnimationBlueprintEditorSettings = GetMutableDefault<UAnimationBlueprintEditorSettings>();
	AnimationBlueprintEditorSettings->UnregisterOnUpdateSettings(AnimationBlueprintEditorSettingsChangedHandle);

	// Remove all Pose Watches that were created as a result of selection, otherwise if the editor options are changed 
	// they will still be active if we get recreated even though the nodes won't be selected.
	RemoveAllSelectionPoseWatches();

	GEditor->OnBlueprintPreCompile().RemoveAll(this);

	GEditor->GetEditorSubsystem<UImportSubsystem>()->OnAssetPostImport.RemoveAll(this);
	FReimportManager::Instance()->OnPostReimport().RemoveAll(this);

	// NOTE: Any tabs that we still have hanging out when destroyed will be cleaned up by FBaseToolkit's destructor

	SaveEditorSettings();

	// Explicit Reset of the PersonaToolKit to force destruction of the PreviewScene
	// This will call PreviewWorld->CleanupWorld() while the EditorModeManager still has the PreviewScene
	PersonaToolkit.Reset();

	// Now we have to clean the PersonaToolkit PreviewScene from the EditorModeManager, as it has been destroyed.
	// This avoids a memory after delete use when any additional PreviewScene calls PreviewWorld->CleanupWorld(),
	// as that function executes a callback that the EditorModeManager is registered to.
	FEditorModeTools& ModeTools = GetEditorModeManager();
	((FAssetEditorModeManager&)ModeTools).SetPreviewScene(nullptr);

	FBlueprintCoreDelegates::OnScriptException.Remove(ScriptExceptionHandle);
}

void FAnimationBlueprintEditor::HandleUpdateSettings(const UAnimationBlueprintEditorSettings* AnimationBlueprintEditorSettings, EPropertyChangeType::Type ChangeType)
{
	if (AnimationBlueprintEditorSettings->bPoseWatchSelectedNodes != bPreviousPoseWatchSelectedNodes)
	{
		bPreviousPoseWatchSelectedNodes = AnimationBlueprintEditorSettings->bPoseWatchSelectedNodes;
		RemoveAllSelectionPoseWatches();
		if (AnimationBlueprintEditorSettings->bPoseWatchSelectedNodes)
		{
			HandlePoseWatchSelectedNodes();
		}
	}
}

UAnimBlueprint* FAnimationBlueprintEditor::GetAnimBlueprint() const
{
	return Cast<UAnimBlueprint>(GetBlueprintObj());
}

void FAnimationBlueprintEditor::ExtendMenu()
{
	if(MenuExtender.IsValid())
	{
		RemoveMenuExtender(MenuExtender);
		MenuExtender.Reset();
	}

	MenuExtender = MakeShareable(new FExtender);
	AddMenuExtender(MenuExtender);

	// add extensible menu if exists
	FAnimationBlueprintEditorModule& AnimationBlueprintEditorModule = FModuleManager::LoadModuleChecked<FAnimationBlueprintEditorModule>("AnimationBlueprintEditor");
	AddMenuExtender(AnimationBlueprintEditorModule.GetMenuExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	FToolMenuOwnerScoped OwnerScoped(this);

	// Add in Editor Specific functionality
	static const FName MenuName = GetToolMenuName();
	static const FName ToolsMenuName = *(MenuName.ToString() + TEXT(".") + TEXT("Tools"));

	UToolMenu* ToolsMenu = UToolMenus::Get()->ExtendMenu(ToolsMenuName);
	const FToolMenuInsert SectionInsertLocation("Programming", EToolMenuInsertType::Before);

	UAnimBlueprint* AnimBlueprint = PersonaToolkit->GetAnimBlueprint();
	if (AnimBlueprint && AnimBlueprint->BlueprintType != BPTYPE_Interface && !AnimBlueprint->bIsTemplate)
	{
		ToolsMenu->AddDynamicSection("Persona", FNewToolMenuDelegate::CreateLambda([](UToolMenu* InToolMenu)
		{
			TSharedPtr<FAnimationBlueprintEditor> AnimationBlueprintEditor = GetAnimationBlueprintEditor(InToolMenu->Context);
			if (AnimationBlueprintEditor.IsValid() && AnimationBlueprintEditor->PersonaToolkit.IsValid())
			{
				FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
				FPersonaModule::FCommonToolbarExtensionArgs Args;
				Args.bPreviewAnimation = false;
				Args.bPreviewMesh = true;
				Args.bReferencePose = false;
				Args.bCreateAsset = true;
				PersonaModule.AddCommonMenuExtensions(InToolMenu, Args);
			}
		}), SectionInsertLocation);
	}
}

void FAnimationBlueprintEditor::RegisterMenus()
{
	FBlueprintEditor::RegisterMenus();
}

void FAnimationBlueprintEditor::InitAnimationBlueprintEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, UAnimBlueprint* InAnimBlueprint)
{
	// Record if we have been newly created
	bool bNewlyCreated = InAnimBlueprint->bIsNewlyCreated;
	InAnimBlueprint->bIsNewlyCreated = false;

	if (!Toolbar.IsValid())
	{
		Toolbar = MakeShareable(new FBlueprintEditorToolbar(SharedThis(this)));
	}

	LoadEditorSettings();

	GetToolkitCommands()->Append(FPlayWorldCommands::GlobalPlayWorldActions.ToSharedRef());

	FPersonaModule& PersonaModule = FModuleManager::GetModuleChecked<FPersonaModule>("Persona");
	PersonaToolkit = PersonaModule.CreatePersonaToolkit(InAnimBlueprint, FPersonaToolkitArgs());

	PersonaToolkit->GetPreviewScene()->SetDefaultAnimationMode(EPreviewSceneDefaultAnimationMode::AnimationBlueprint);
	PersonaToolkit->GetPreviewScene()->RegisterOnPreviewMeshChanged(FOnPreviewMeshChanged::CreateSP(this, &FAnimationBlueprintEditor::HandlePreviewMeshChanged));

	// leave some metadata on the world used for debug object labeling
	if(FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(PersonaToolkit->GetPreviewScene()->GetWorld()))
	{
		static constexpr TCHAR Format[] = TEXT("AnimBPEditor (%s)");
		WorldContext->CustomDescription = FString::Printf(Format, *InAnimBlueprint->GetName());
	}

	PersonaModule.RecordAssetOpened(InAnimBlueprint);

	if(InAnimBlueprint->BlueprintType != BPTYPE_Interface && !InAnimBlueprint->bIsTemplate)
	{
		// create the skeleton tree
		FSkeletonTreeArgs SkeletonTreeArgs;
		SkeletonTreeArgs.OnSelectionChanged = FOnSkeletonTreeSelectionChanged::CreateSP(this, &FAnimationBlueprintEditor::HandleSelectionChanged);
		SkeletonTreeArgs.PreviewScene = GetPreviewScene();
		SkeletonTreeArgs.ContextName = GetToolkitFName();

		ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
		SkeletonTree = SkeletonEditorModule.CreateSkeletonTree(PersonaToolkit->GetSkeleton(), SkeletonTreeArgs);
	}

	// Register for compilation events
	InAnimBlueprint->OnCompiled().AddSP(this, &FAnimationBlueprintEditor::OnBlueprintPostCompile);

	// Build up a list of objects being edited in this asset editor
	TArray<UObject*> ObjectsBeingEdited;
	ObjectsBeingEdited.Add(InAnimBlueprint);

	CreateDefaultCommands();

	BindCommands();

	RegisterMenus();

	// Initialize the asset editor and spawn tabs
	const bool bCreateDefaultStandaloneMenu = true;
	const bool bCreateDefaultToolbar = true;
	InitAssetEditor(Mode, InitToolkitHost, AnimationBlueprintEditorAppName, FTabManager::FLayout::NullLayout, bCreateDefaultStandaloneMenu, bCreateDefaultToolbar, ObjectsBeingEdited);

	TArray<UBlueprint*> AnimBlueprints;
	AnimBlueprints.Add(InAnimBlueprint);

	CommonInitialization(AnimBlueprints, /*bShouldOpenInDefaultsMode=*/ false);

	// Register document editor for blendspaces
	DocumentManager->RegisterDocumentFactory(MakeShared<FBlendSpaceDocumentTabFactory>(SharedThis(this)));

	bool bHasBlueprintPreview = false;

	if(InAnimBlueprint->BlueprintType == BPTYPE_Interface)
	{
		AddApplicationMode(
			FAnimationBlueprintEditorModes::AnimationBlueprintInterfaceEditorMode,
			MakeShareable(new FAnimationBlueprintInterfaceEditorMode(SharedThis(this))));

		ExtendMenu();
		ExtendToolbar();
		RegenerateMenusAndToolbars();

		// Activate the initial mode (which will populate with a real layout)
		SetCurrentMode(FAnimationBlueprintEditorModes::AnimationBlueprintInterfaceEditorMode);
	}
	else if(InAnimBlueprint->bIsTemplate)
	{
		AddApplicationMode(
			FAnimationBlueprintEditorModes::AnimationBlueprintTemplateEditorMode,
			MakeShareable(new FAnimationBlueprintEditorMode(SharedThis(this))));

		bHasBlueprintPreview = true;
		
		ExtendMenu();
		ExtendToolbar();
		RegenerateMenusAndToolbars();

		// Activate the initial mode (which will populate with a real layout)
		SetCurrentMode(FAnimationBlueprintEditorModes::AnimationBlueprintTemplateEditorMode);
	}
	else
	{
		AddApplicationMode(
			FAnimationBlueprintEditorModes::AnimationBlueprintEditorMode,
			MakeShareable(new FAnimationBlueprintEditorMode(SharedThis(this))));

		bHasBlueprintPreview = true;

		ExtendMenu();
		ExtendToolbar();
		RegenerateMenusAndToolbars();

		// Activate the initial mode (which will populate with a real layout)
		SetCurrentMode(FAnimationBlueprintEditorModes::AnimationBlueprintEditorMode);
	}

	if (bHasBlueprintPreview)
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = PersonaToolkit->GetPreviewMeshComponent();
		UAnimBlueprint* AnimBlueprint = PersonaToolkit->GetAnimBlueprint();
		UAnimBlueprint* PreviewAnimBlueprint = AnimBlueprint->GetPreviewAnimationBlueprint();

		if (PreviewAnimBlueprint)
		{
			PersonaToolkit->GetPreviewScene()->SetPreviewAnimationBlueprint(PreviewAnimBlueprint, AnimBlueprint);
			PreviewAnimBlueprint->OnCompiled().AddSP(this, &FAnimationBlueprintEditor::HandlePreviewAnimBlueprintCompiled);
		}
		else
		{
			PersonaToolkit->GetPreviewScene()->SetPreviewAnimationBlueprint(AnimBlueprint, nullptr);
		}

		PersonaUtils::SetObjectBeingDebugged(AnimBlueprint, PreviewMeshComponent->GetAnimInstance());
	}

	// Post-layout initialization
	PostLayoutBlueprintEditorInitialization();

	// register customization of Slot node for this Animation Blueprint Editor
	// this is so that you can open the manage window per Animation Blueprint Editor
	PersonaModule.CustomizeBlueprintEditorDetails(Inspector->GetPropertyView().ToSharedRef(), FOnInvokeTab::CreateSP(this, &FAssetEditorToolkit::InvokeTab));

	if(bNewlyCreated && InAnimBlueprint->BlueprintType == BPTYPE_Interface)
	{
		NewDocument_OnClick(CGT_NewAnimationLayer);
	}

	// Register for notifications when settings change
	AnimationBlueprintEditorSettingsChangedHandle = GetMutableDefault<UAnimationBlueprintEditorSettings>()->RegisterOnUpdateSettings(
		UAnimationBlueprintEditorSettings::FOnUpdateSettingsMulticaster::FDelegate::CreateSP(this, &FAnimationBlueprintEditor::HandleUpdateSettings));

	PersonaToolkit->GetPreviewScene()->SetAllowMeshHitProxies(false);

	ScriptExceptionHandle = FBlueprintCoreDelegates::OnScriptException.AddSP(this, &FAnimationBlueprintEditor::HandleScriptException);
}

void FAnimationBlueprintEditor::BindCommands()
{
	GetToolkitCommands()->MapAction(FPersonaCommonCommands::Get().TogglePlay,
		FExecuteAction::CreateRaw(&GetPersonaToolkit()->GetPreviewScene().Get(), &IPersonaPreviewScene::TogglePlayback));
}

TSharedPtr<FAnimationBlueprintEditor> FAnimationBlueprintEditor::GetAnimationBlueprintEditor(const FToolMenuContext& InMenuContext)
{
	if (UAnimationBlueprintToolMenuContext* Context = InMenuContext.FindContext<UAnimationBlueprintToolMenuContext>())
	{
		if (Context->AnimationBlueprintEditor.IsValid())
		{
			return StaticCastSharedPtr<FAnimationBlueprintEditor>(Context->AnimationBlueprintEditor.Pin());
		}
	}

	return TSharedPtr<FAnimationBlueprintEditor>();
}

void FAnimationBlueprintEditor::ExtendToolbar()
{
	// If the ToolbarExtender is valid, remove it before rebuilding it
	if(ToolbarExtender.IsValid())
	{
		RemoveToolbarExtender(ToolbarExtender);
		ToolbarExtender.Reset();
	}

	ToolbarExtender = MakeShareable(new FExtender);

	AddToolbarExtender(ToolbarExtender);

	FAnimationBlueprintEditorModule& AnimationBlueprintEditorModule = FModuleManager::LoadModuleChecked<FAnimationBlueprintEditorModule>("AnimationBlueprintEditor");
	AddToolbarExtender(AnimationBlueprintEditorModule.GetToolBarExtensibilityManager()->GetAllExtenders(GetToolkitCommands(), GetEditingObjects()));

	TArray<IAnimationBlueprintEditorModule::FAnimationBlueprintEditorToolbarExtender> ToolbarExtenderDelegates = AnimationBlueprintEditorModule.GetAllAnimationBlueprintEditorToolbarExtenders();

	for (auto& ToolbarExtenderDelegate : ToolbarExtenderDelegates)
	{
		if(ToolbarExtenderDelegate.IsBound())
		{
			AddToolbarExtender(ToolbarExtenderDelegate.Execute(GetToolkitCommands(), SharedThis(this)));
		}
	}

	UAnimBlueprint* AnimBlueprint = PersonaToolkit->GetAnimBlueprint();
	if(AnimBlueprint && AnimBlueprint->BlueprintType != BPTYPE_Interface && !AnimBlueprint->bIsTemplate)
	{
		ToolbarExtender->AddToolBarExtension(
			"Asset",
			EExtensionHook::After,
			GetToolkitCommands(),
			FToolBarExtensionDelegate::CreateLambda([this](FToolBarBuilder& ParentToolbarBuilder)
			{
				FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");
				TSharedRef<class IAssetFamily> AssetFamily = PersonaModule.CreatePersonaAssetFamily(GetBlueprintObj());
				AddToolbarWidget(PersonaModule.CreateAssetFamilyShortcutWidget(SharedThis(this), AssetFamily));
			}
		));
	}
}

UBlueprint* FAnimationBlueprintEditor::GetBlueprintObj() const
{
	const TArray<UObject*>& EditingObjs = GetEditingObjects();
	for (int32 i = 0; i < EditingObjs.Num(); ++i)
	{
		if (EditingObjs[i]->IsA<UAnimBlueprint>()) {return (UBlueprint*)EditingObjs[i];}
	}
	return nullptr;
}

void FAnimationBlueprintEditor::SetDetailObjects(const TArray<UObject*>& InObjects)
{
	Inspector->ShowDetailsForObjects(InObjects);
}

void FAnimationBlueprintEditor::SetDetailObject(UObject* Obj)
{
	TArray<UObject*> Objects;
	if (Obj)
	{
		Objects.Add(Obj);
	}
	SetDetailObjects(Objects);
}

/** Called when graph editor focus is changed */
void FAnimationBlueprintEditor::OnGraphEditorFocused(const TSharedRef<class SGraphEditor>& InGraphEditor)
{
	// Remove pose watches now before calling the base class implementation because that will switch the focus
	if (GetDefault<UAnimationBlueprintEditorSettings>()->bPoseWatchSelectedNodes)
	{
		RemoveAllSelectionPoseWatches();
	}

	// in the future, depending on which graph editor is this will act different
	FBlueprintEditor::OnGraphEditorFocused(InGraphEditor);

	// install callback to allow us to propagate pin default changes live to the preview
	UAnimationGraph* AnimationGraph = Cast<UAnimationGraph>(InGraphEditor->GetCurrentGraph());
	if (AnimationGraph)
	{
		OnPinDefaultValueChangedHandle = AnimationGraph->OnPinDefaultValueChanged.Add(FOnPinDefaultValueChanged::FDelegate::CreateSP(this, &FAnimationBlueprintEditor::HandlePinDefaultValueChanged));
	}

	if (bHideUnrelatedNodes && GetSelectedNodes().Num() <= 0)
	{
		ResetAllNodesUnrelatedStates();
	}

	if (GetDefault<UAnimationBlueprintEditorSettings>()->bPoseWatchSelectedNodes)
	{
		HandlePoseWatchSelectedNodes();
	}
}

void FAnimationBlueprintEditor::OnGraphEditorBackgrounded(const TSharedRef<SGraphEditor>& InGraphEditor)
{
	FBlueprintEditor::OnGraphEditorBackgrounded(InGraphEditor);

	UAnimationGraph* AnimationGraph = Cast<UAnimationGraph>(InGraphEditor->GetCurrentGraph());
	if (AnimationGraph)
	{
		AnimationGraph->OnPinDefaultValueChanged.Remove(OnPinDefaultValueChangedHandle);
	}
}

/** Create Default Tabs **/
void FAnimationBlueprintEditor::CreateDefaultCommands() 
{
	{
		FBlueprintEditor::CreateDefaultCommands();
	}
}

void FAnimationBlueprintEditor::OnCreateGraphEditorCommands(TSharedPtr<FUICommandList> GraphEditorCommandsList)
{
	GraphEditorCommandsList->MapAction(FAnimGraphCommands::Get().TogglePoseWatch,
		FExecuteAction::CreateSP(this, &FAnimationBlueprintEditor::OnTogglePoseWatch),
		FCanExecuteAction::CreateSP(this, &FAnimationBlueprintEditor::CanTogglePoseWatch)
	);

	GraphEditorCommandsList->MapAction(FAnimGraphCommands::Get().HideUnboundPropertyPins,
		FExecuteAction::CreateSP(this, &FAnimationBlueprintEditor::OnHideUnboundPropertyPins),
		FCanExecuteAction::CreateSP(this, &FAnimationBlueprintEditor::CanHideUnboundPropertyPins)
	);

	GraphEditorCommandsList->MapAction( FAnimGraphCommands::Get().AddBlendListPin,
		FExecuteAction::CreateSP( this, &FAnimationBlueprintEditor::OnAddPosePin ),
		FCanExecuteAction::CreateSP( this, &FAnimationBlueprintEditor::CanAddPosePin )
		);

	GraphEditorCommandsList->MapAction( FAnimGraphCommands::Get().RemoveBlendListPin,
		FExecuteAction::CreateSP( this, &FAnimationBlueprintEditor::OnRemovePosePin ),
		FCanExecuteAction::CreateSP( this, &FAnimationBlueprintEditor::CanRemovePosePin )
		);

	GraphEditorCommandsList->MapAction( FAnimGraphCommands::Get().ConvertToSeqEvaluator,
		FExecuteAction::CreateSP( this, &FAnimationBlueprintEditor::OnConvertToSequenceEvaluator )
		);

	GraphEditorCommandsList->MapAction( FAnimGraphCommands::Get().ConvertToSeqPlayer,
		FExecuteAction::CreateSP( this, &FAnimationBlueprintEditor::OnConvertToSequencePlayer )
		);

	GraphEditorCommandsList->MapAction( FAnimGraphCommands::Get().ConvertToBSEvaluator,
		FExecuteAction::CreateSP( this, &FAnimationBlueprintEditor::OnConvertToBlendSpaceEvaluator )
		);

	GraphEditorCommandsList->MapAction( FAnimGraphCommands::Get().ConvertToBSPlayer,
		FExecuteAction::CreateSP( this, &FAnimationBlueprintEditor::OnConvertToBlendSpacePlayer )
		);

	GraphEditorCommandsList->MapAction( FAnimGraphCommands::Get().ConvertToBSGraph,
		FExecuteAction::CreateSP( this, &FAnimationBlueprintEditor::OnConvertToBlendSpaceGraph )
		);

	GraphEditorCommandsList->MapAction(FAnimGraphCommands::Get().ConvertToAimOffsetLookAt,
		FExecuteAction::CreateSP(this, &FAnimationBlueprintEditor::OnConvertToAimOffsetLookAt)
	);

	GraphEditorCommandsList->MapAction(FAnimGraphCommands::Get().ConvertToAimOffsetSimple,
		FExecuteAction::CreateSP(this, &FAnimationBlueprintEditor::OnConvertToAimOffsetSimple)
	);

	GraphEditorCommandsList->MapAction(FAnimGraphCommands::Get().ConvertToAimOffsetGraph,
		FExecuteAction::CreateSP(this, &FAnimationBlueprintEditor::OnConvertToAimOffsetGraph)
	);	

	GraphEditorCommandsList->MapAction(FAnimGraphCommands::Get().ConvertToPoseBlender,
		FExecuteAction::CreateSP(this, &FAnimationBlueprintEditor::OnConvertToPoseBlender)
		);

	GraphEditorCommandsList->MapAction(FAnimGraphCommands::Get().ConvertToPoseByName,
		FExecuteAction::CreateSP(this, &FAnimationBlueprintEditor::OnConvertToPoseByName)
		);

	GraphEditorCommandsList->MapAction( FAnimGraphCommands::Get().OpenRelatedAsset,
		FExecuteAction::CreateSP( this, &FAnimationBlueprintEditor::OnOpenRelatedAsset )
		);

	GraphEditorCommandsList->MapAction( FAnimGraphCommands::Get().CreateSelfTransition,
		FExecuteAction::CreateSP( this, &FAnimationBlueprintEditor::OnCreateSelfTransition )
		);

	GraphEditorCommandsList->MapAction( FAnimGraphCommands::Get().SetActiveState,
		FUIAction(
			FExecuteAction::CreateSP( this, &FAnimationBlueprintEditor::OnSetActiveState ),
			FCanExecuteAction::CreateSP( this, &FAnimationBlueprintEditor::CanSetActiveState )
			)
		);

	GraphEditorCommandsList->MapAction( FAnimGraphCommands::Get().ReverseTransition,
		FExecuteAction::CreateSP( this, &FAnimationBlueprintEditor::OnReverseTransition )
		);
}


void FAnimationBlueprintEditor::OnAddPosePin()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() == 1)
	{
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			UObject* Node = *NodeIt;

			if (UAnimGraphNode_BlendListByInt* BlendNode = Cast<UAnimGraphNode_BlendListByInt>(Node))
			{
				BlendNode->AddPinToBlendList();
				break;
			}
			else if (UAnimGraphNode_LayeredBoneBlend* FilterNode = Cast<UAnimGraphNode_LayeredBoneBlend>(Node))
			{
				FilterNode->AddPinToBlendByFilter();
				break;
			}
			else if (UAnimGraphNode_MultiWayBlend* MultiBlendNode = Cast<UAnimGraphNode_MultiWayBlend>(Node))
			{
				MultiBlendNode->AddPinToBlendNode();
				break;
			}
		}
	}
}

bool FAnimationBlueprintEditor::CanAddPosePin() const
{
	return true;
}

void FAnimationBlueprintEditor::OnRemovePosePin()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	UAnimGraphNode_BlendListByInt* BlendListIntNode = nullptr;
	UAnimGraphNode_LayeredBoneBlend* BlendByFilterNode = nullptr;
	UAnimGraphNode_MultiWayBlend* BlendByMultiway = nullptr;

	if (SelectedNodes.Num() == 1)
	{
		for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
		{
			if (UAnimGraphNode_BlendListByInt* BlendNode = Cast<UAnimGraphNode_BlendListByInt>(*NodeIt))
			{
				BlendListIntNode = BlendNode;
				break;
			}
			else if (UAnimGraphNode_LayeredBoneBlend* LayeredBlendNode = Cast<UAnimGraphNode_LayeredBoneBlend>(*NodeIt))
			{
				BlendByFilterNode = LayeredBlendNode;
				break;
			}		
			else if (UAnimGraphNode_MultiWayBlend* MultiwayBlendNode = Cast<UAnimGraphNode_MultiWayBlend>(*NodeIt))
			{
				BlendByMultiway = MultiwayBlendNode;
				break;
			}
		}
	}


	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		// @fixme: I think we can make blendlistbase to have common functionality
		// and each can implement the common function, but for now, we separate them
		// each implement their menu, so we still can use listbase as the root
		if (BlendListIntNode)
		{
			// make sure we at least have BlendListNode selected
			UEdGraphPin* SelectedPin = FocusedGraphEd->GetGraphPinForMenu();

			BlendListIntNode->RemovePinFromBlendList(SelectedPin);

			// Update the graph so that the node will be refreshed
			FocusedGraphEd->NotifyGraphChanged();
		}

		if (BlendByFilterNode)
		{
			// make sure we at least have BlendListNode selected
			UEdGraphPin* SelectedPin = FocusedGraphEd->GetGraphPinForMenu();

			BlendByFilterNode->RemovePinFromBlendByFilter(SelectedPin);

			// Update the graph so that the node will be refreshed
			FocusedGraphEd->NotifyGraphChanged();
		}

		if (BlendByMultiway)
		{
			// make sure we at least have BlendListNode selected
			UEdGraphPin* SelectedPin = FocusedGraphEd->GetGraphPinForMenu();

			BlendByMultiway->RemovePinFromBlendNode(SelectedPin);

			// Update the graph so that the node will be refreshed
			FocusedGraphEd->NotifyGraphChanged();
		}
	}
}

bool FAnimationBlueprintEditor::CanTogglePoseWatch()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	UAnimBlueprint* AnimBP = GetAnimBlueprint();

	// Can't add pose watch from a child anim bp
	if (UAnimBlueprint::FindRootAnimBlueprint(AnimBP))
	{
		return false;
	}

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (UAnimGraphNode_Base* SelectedNode = Cast<UAnimGraphNode_Base>(*NodeIt))
		{
			UPoseWatch* ExistingPoseWatch = AnimationEditorUtils::FindPoseWatchForNode(SelectedNode, AnimBP);
			if (ExistingPoseWatch)
			{
				return true;
			}
			if (SelectedNode->IsPoseWatchable())
			{
				return true;
			}
		}
	}

	return false;
}

bool FAnimationBlueprintEditor::CanHideUnboundPropertyPins()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (UAnimGraphNode_LinkedAnimGraphBase* SelectedNode = Cast<UAnimGraphNode_LinkedAnimGraphBase>(*NodeIt))
		{
			return true;
		}
	}

	return false;
}

void FAnimationBlueprintEditor::OnTogglePoseWatch()
{
	ReleaseAllManagedNodes();

	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	UAnimBlueprint* AnimBP = GetAnimBlueprint();

	// Can't toggle pose watch from a child anim bp
	if (UAnimBlueprint::FindRootAnimBlueprint(AnimBP))
	{
		return;
	}

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (UAnimGraphNode_Base* SelectedNode = Cast<UAnimGraphNode_Base>(*NodeIt))
		{
			UPoseWatch* ExistingPoseWatch = AnimationEditorUtils::FindPoseWatchForNode(SelectedNode, AnimBP);
			if (ExistingPoseWatch)
			{
				// Promote the temporary pose watch to permanent 
				if (ExistingPoseWatch->GetShouldDeleteOnDeselect())
				{
					ExistingPoseWatch->SetShouldDeleteOnDeselect(false);
				}
				else if (GetDefault<UAnimationBlueprintEditorSettings>()->bPoseWatchSelectedNodes)
				{
					ExistingPoseWatch->SetShouldDeleteOnDeselect(true);
				}
				else
				{
					AnimationEditorUtils::RemovePoseWatch(ExistingPoseWatch, AnimBP);
				}
				AnimationEditorUtils::OnPoseWatchesChanged().Broadcast(AnimBP, ExistingPoseWatch->Node.Get());
			}
			else if (SelectedNode->IsPoseWatchable())
			{
				UPoseWatch* NewPoseWatch = AnimationEditorUtils::MakePoseWatchForNode(AnimBP, SelectedNode);
				AnimationEditorUtils::OnPoseWatchesChanged().Broadcast(AnimBP, NewPoseWatch->Node.Get());
			}
		}
	}

	AcquireAllManagedNodes();
}

void FAnimationBlueprintEditor::OnHideUnboundPropertyPins()
{
	const FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	for (FGraphPanelSelectionSet::TConstIterator NodeIt(SelectedNodes); NodeIt; ++NodeIt)
	{
		if (UAnimGraphNode_LinkedAnimGraphBase* SelectedNode = Cast<UAnimGraphNode_LinkedAnimGraphBase>(*NodeIt))
		{
			UAnimationGraphSchema::HideUnboundPropertyPins(SelectedNode);
		}
	}
}

// Helper function for node conversions
static void CopyPinData(UEdGraphNode* InOldNode, UEdGraphNode* InNewNode, const TCHAR* InPinName)
{
	UEdGraphPin* OldPin = InOldNode->FindPin(InPinName);
	UEdGraphPin* NewPin = InNewNode->FindPin(InPinName);

	if (ensure(OldPin && NewPin))
	{
		NewPin->MovePersistentDataFromOldPin(*OldPin);
	}
};

void FAnimationBlueprintEditor::OnConvertToSequenceEvaluator()
{
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	if (SelectedNodes.Num() > 0)
	{
		// convert to sequence evaluator
		const FScopedTransaction Transaction( LOCTEXT("ConvertToSequenceEvaluator", "Convert to Single Frame Animation") );

		for (auto NodeIter = SelectedNodes.CreateIterator(); NodeIter; ++NodeIter)
		{
			UAnimGraphNode_SequencePlayer* OldNode = Cast<UAnimGraphNode_SequencePlayer>(*NodeIter);

			// see if sequence player
			if ( OldNode && OldNode->Node.GetSequence() )
			{
				UEdGraph* TargetGraph = OldNode->GetGraph();
				TargetGraph->Modify();
				OldNode->Modify();

				// create new evaluator
				FGraphNodeCreator<UAnimGraphNode_SequenceEvaluator> NodeCreator(*TargetGraph);
				UAnimGraphNode_SequenceEvaluator* NewNode = NodeCreator.CreateNode();
				NewNode->Node.SetSequence(OldNode->Node.GetSequence());
				NodeCreator.Finalize();

				// get default data from old node to new node
				FEdGraphUtilities::CopyCommonState(OldNode, NewNode);

				CopyPinData(OldNode, NewNode, TEXT("Pose"));

				// remove from selection and from graph
				NodeIter.RemoveCurrent();
				TargetGraph->RemoveNode(OldNode);
			}
		}

		// @todo fixme: below code doesn't work
		// because of SetAndCenterObject kicks in after new node is added
		// will need to disable that first
		TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();

		// Update the graph so that the node will be refreshed
		FocusedGraphEd->NotifyGraphChanged();
		// It's possible to leave invalid objects in the selection set if they get GC'd, so clear it out
		FocusedGraphEd->ClearSelectionSet();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetAnimBlueprint());
	}
}

void FAnimationBlueprintEditor::OnConvertToSequencePlayer()
{
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() > 0)
	{
		const FScopedTransaction Transaction( LOCTEXT("ConvertToSequencePlayer", "Convert to Sequence Player") );

		for (auto NodeIter = SelectedNodes.CreateIterator(); NodeIter; ++NodeIter)
		{
			UAnimGraphNode_SequenceEvaluator* OldNode = Cast<UAnimGraphNode_SequenceEvaluator>(*NodeIter);

			// see if sequence player
			if ( OldNode && OldNode->Node.GetSequence() )
			{
				// convert to sequence player
				UEdGraph* TargetGraph = OldNode->GetGraph();
				TargetGraph->Modify();
				OldNode->Modify();

				// create new player
				FGraphNodeCreator<UAnimGraphNode_SequencePlayer> NodeCreator(*TargetGraph);
				UAnimGraphNode_SequencePlayer* NewNode = NodeCreator.CreateNode();
				NewNode->Node.SetSequence(OldNode->Node.GetSequence());
				NodeCreator.Finalize();

				// get default data from old node to new node
				FEdGraphUtilities::CopyCommonState(OldNode, NewNode);

				CopyPinData(OldNode, NewNode, TEXT("Pose"));

				// remove from selection and from graph
				NodeIter.RemoveCurrent();
				TargetGraph->RemoveNode(OldNode);
			}
		}

		// @todo fixme: below code doesn't work
		// because of SetAndCenterObject kicks in after new node is added
		// will need to disable that first
		TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();

		// Update the graph so that the node will be refreshed
		FocusedGraphEd->NotifyGraphChanged();
		// It's possible to leave invalid objects in the selection set if they get GC'd, so clear it out
		FocusedGraphEd->ClearSelectionSet();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetAnimBlueprint());
	}
}

void FAnimationBlueprintEditor::OnConvertToBlendSpaceEvaluator() 
{
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	if (SelectedNodes.Num() > 0)
	{
		const FScopedTransaction Transaction( LOCTEXT("ConvertToBlendSpaceEvaluator", "Convert to Single Frame Blend Space") );

		for (auto NodeIter = SelectedNodes.CreateIterator(); NodeIter; ++NodeIter)
		{
			UAnimGraphNode_BlendSpacePlayer* OldNode = Cast<UAnimGraphNode_BlendSpacePlayer>(*NodeIter);

			// see if sequence player
			if ( OldNode && OldNode->Node.GetBlendSpace() )
			{
				// convert to sequence evaluator
				UEdGraph* TargetGraph = OldNode->GetGraph();
				TargetGraph->Modify();
				OldNode->Modify();

				// create new evaluator
				FGraphNodeCreator<UAnimGraphNode_BlendSpaceEvaluator> NodeCreator(*TargetGraph);
				UAnimGraphNode_BlendSpaceEvaluator* NewNode = NodeCreator.CreateNode();
				NewNode->Node.SetBlendSpace(OldNode->Node.GetBlendSpace());
				NodeCreator.Finalize();

				// get default data from old node to new node
				FEdGraphUtilities::CopyCommonState(OldNode, NewNode);

				CopyPinData(OldNode, NewNode, TEXT("X"));
				CopyPinData(OldNode, NewNode, TEXT("Y"));
				CopyPinData(OldNode, NewNode, TEXT("Pose"));

				// remove from selection and from graph
				NodeIter.RemoveCurrent();
				TargetGraph->RemoveNode(OldNode);
			}
		}

		// @todo fixme: below code doesn't work
		// because of SetAndCenterObject kicks in after new node is added
		// will need to disable that first
		TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();

		// Update the graph so that the node will be refreshed
		FocusedGraphEd->NotifyGraphChanged();
		// It's possible to leave invalid objects in the selection set if they get GC'd, so clear it out
		FocusedGraphEd->ClearSelectionSet();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetAnimBlueprint());
	}
}
void FAnimationBlueprintEditor::OnConvertToBlendSpacePlayer() 
{
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() > 0)
	{
		const FScopedTransaction Transaction( LOCTEXT("ConvertToBlendSpacePlayer", "Convert to Blend Space Player") );

		for (auto NodeIter = SelectedNodes.CreateIterator(); NodeIter; ++NodeIter)
		{
			UAnimGraphNode_BlendSpaceEvaluator* OldNode = Cast<UAnimGraphNode_BlendSpaceEvaluator>(*NodeIter);

			// see if sequence player
			if ( OldNode && OldNode->Node.GetBlendSpace() )
			{
				// convert to sequence player
				UEdGraph* TargetGraph = OldNode->GetGraph();
				TargetGraph->Modify();
				OldNode->Modify();

				// create new player
				FGraphNodeCreator<UAnimGraphNode_BlendSpacePlayer> NodeCreator(*TargetGraph);
				UAnimGraphNode_BlendSpacePlayer* NewNode = NodeCreator.CreateNode();
				NewNode->Node.SetBlendSpace(OldNode->Node.GetBlendSpace());
				NodeCreator.Finalize();

				// get default data from old node to new node
				FEdGraphUtilities::CopyCommonState(OldNode, NewNode);

				CopyPinData(OldNode, NewNode, TEXT("X"));
				CopyPinData(OldNode, NewNode, TEXT("Y"));
				CopyPinData(OldNode, NewNode, TEXT("Pose"));

				// remove from selection and from graph
				NodeIter.RemoveCurrent();
				TargetGraph->RemoveNode(OldNode);
			}
		}

		// @todo fixme: below code doesn't work
		// because of SetAndCenterObject kicks in after new node is added
		// will need to disable that first
		TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
		// Update the graph so that the node will be refreshed
		FocusedGraphEd->NotifyGraphChanged();
		// It's possible to leave invalid objects in the selection set if they get GC'd, so clear it out
		FocusedGraphEd->ClearSelectionSet();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetAnimBlueprint());
	}
}

void FAnimationBlueprintEditor::OnConvertToBlendSpaceGraph()
{
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() > 0)
	{
		const FScopedTransaction Transaction( LOCTEXT("ConvertToblendSpaceGraph", "Convert to Blend Space Graph") );

		for (auto NodeIter = SelectedNodes.CreateIterator(); NodeIter; ++NodeIter)
		{
			UAnimGraphNode_BlendSpacePlayer* OldNode = Cast<UAnimGraphNode_BlendSpacePlayer>(*NodeIter);

			// see if sequence player
			if (OldNode && OldNode->Node.GetBlendSpace())
			{
				// convert to sequence player
				UEdGraph* TargetGraph = OldNode->GetGraph();
				TargetGraph->Modify();
				OldNode->Modify();

				// create new player
				FGraphNodeCreator<UAnimGraphNode_BlendSpaceGraph> NodeCreator(*TargetGraph);
				UAnimGraphNode_BlendSpaceGraph* NewNode = NodeCreator.CreateNode();
				if(OldNode->Node.GetGroupName() != NAME_None && OldNode->Node.GetGroupMethod() == EAnimSyncMethod::SyncGroup)
				{
					NewNode->SetSyncGroupName(OldNode->Node.GetGroupName());
				}
				NewNode->SetupFromAsset(OldNode->Node.GetBlendSpace(), false);
				NodeCreator.Finalize();

				// get default data from old node to new node
				FEdGraphUtilities::CopyCommonState(OldNode, NewNode);

				CopyPinData(OldNode, NewNode, TEXT("X"));
				CopyPinData(OldNode, NewNode, TEXT("Y"));
				CopyPinData(OldNode, NewNode, TEXT("Pose"));

				// remove from selection and from graph
				NodeIter.RemoveCurrent();
				TargetGraph->RemoveNode(OldNode);
			}
		}

		// @todo fixme: below code doesn't work
		// because of SetAndCenterObject kicks in after new node is added
		// will need to disable that first
		TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
		// Update the graph so that the node will be refreshed
		FocusedGraphEd->NotifyGraphChanged();
		// It's possible to leave invalid objects in the selection set if they get GC'd, so clear it out
		FocusedGraphEd->ClearSelectionSet();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetAnimBlueprint());
	}
}

void FAnimationBlueprintEditor::OnConvertToPoseBlender()
{
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() > 0)
	{
		const FScopedTransaction Transaction( LOCTEXT("ConvertToPoseBlender", "Convert to Pose Blender") );

		for (auto NodeIter = SelectedNodes.CreateIterator(); NodeIter; ++NodeIter)
		{
			UAnimGraphNode_PoseByName* OldNode = Cast<UAnimGraphNode_PoseByName>(*NodeIter);

			// see if sequence player
			if (OldNode && OldNode->Node.PoseAsset)
			{
				// convert to sequence player
				UEdGraph* TargetGraph = OldNode->GetGraph();
				TargetGraph->Modify();
				OldNode->Modify();

				// create new player
				FGraphNodeCreator<UAnimGraphNode_PoseBlendNode> NodeCreator(*TargetGraph);
				UAnimGraphNode_PoseBlendNode* NewNode = NodeCreator.CreateNode();
				NewNode->Node.PoseAsset = OldNode->Node.PoseAsset;
				NodeCreator.Finalize();

				// get default data from old node to new node
				FEdGraphUtilities::CopyCommonState(OldNode, NewNode);

				CopyPinData(OldNode, NewNode, TEXT("Pose"));

				// remove from selection and from graph
				NodeIter.RemoveCurrent();
				TargetGraph->RemoveNode(OldNode);
			}
		}

		// @todo fixme: below code doesn't work
		// because of SetAndCenterObject kicks in after new node is added
		// will need to disable that first
		TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();

		// Update the graph so that the node will be refreshed
		FocusedGraphEd->NotifyGraphChanged();
		// It's possible to leave invalid objects in the selection set if they get GC'd, so clear it out
		FocusedGraphEd->ClearSelectionSet();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetAnimBlueprint());
	}
}

void FAnimationBlueprintEditor::OnConvertToPoseByName()
{
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() > 0)
	{
		const FScopedTransaction Transaction( LOCTEXT("ConvertToPoseByName", "Convert to Pose By Name") );

		for (auto NodeIter = SelectedNodes.CreateIterator(); NodeIter; ++NodeIter)
		{
			UAnimGraphNode_PoseBlendNode* OldNode = Cast<UAnimGraphNode_PoseBlendNode>(*NodeIter);

			// see if sequence player
			if (OldNode && OldNode->Node.PoseAsset)
			{
				// convert to sequence player
				UEdGraph* TargetGraph = OldNode->GetGraph();
				TargetGraph->Modify();
				OldNode->Modify();

				// create new player
				FGraphNodeCreator<UAnimGraphNode_PoseByName> NodeCreator(*TargetGraph);
				UAnimGraphNode_PoseByName* NewNode = NodeCreator.CreateNode();
				NewNode->Node.PoseAsset = OldNode->Node.PoseAsset;
				NodeCreator.Finalize();

				// get default data from old node to new node
				FEdGraphUtilities::CopyCommonState(OldNode, NewNode);

				CopyPinData(OldNode, NewNode, TEXT("Pose"));

				// remove from selection and from graph
				NodeIter.RemoveCurrent();
				TargetGraph->RemoveNode(OldNode);
			}
		}

		// @todo fixme: below code doesn't work
		// because of SetAndCenterObject kicks in after new node is added
		// will need to disable that first
		TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();

		// Update the graph so that the node will be refreshed
		FocusedGraphEd->NotifyGraphChanged();
		// It's possible to leave invalid objects in the selection set if they get GC'd, so clear it out
		FocusedGraphEd->ClearSelectionSet();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetAnimBlueprint());
	}
}

void FAnimationBlueprintEditor::OnConvertToAimOffsetLookAt()
{
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();

	if (SelectedNodes.Num() > 0)
	{
		const FScopedTransaction Transaction( LOCTEXT("ConvertToAimOffsetLookAt", "Convert to Aim Offset LookAt") );

		for (auto NodeIter = SelectedNodes.CreateIterator(); NodeIter; ++NodeIter)
		{
			UAnimGraphNode_RotationOffsetBlendSpace* OldNode = Cast<UAnimGraphNode_RotationOffsetBlendSpace>(*NodeIter);

			// see if sequence player
			if (OldNode && OldNode->Node.GetBlendSpace())
			{
				// convert to sequence evaluator
				UEdGraph* TargetGraph = OldNode->GetGraph();
				TargetGraph->Modify();
				OldNode->Modify();

				// create new evaluator
				FGraphNodeCreator<UAnimGraphNode_AimOffsetLookAt> NodeCreator(*TargetGraph);
				UAnimGraphNode_AimOffsetLookAt* NewNode = NodeCreator.CreateNode();
				NewNode->Node.SetBlendSpace(OldNode->Node.GetBlendSpace());
				NodeCreator.Finalize();

				// get default data from old node to new node
				FEdGraphUtilities::CopyCommonState(OldNode, NewNode);

				CopyPinData(OldNode, NewNode, TEXT("X"));
				CopyPinData(OldNode, NewNode, TEXT("Y"));
				CopyPinData(OldNode, NewNode, TEXT("Alpha"));
				CopyPinData(OldNode, NewNode, TEXT("Pose"));
				CopyPinData(OldNode, NewNode, TEXT("BasePose"));

				// remove from selection and from graph
				NodeIter.RemoveCurrent();
				TargetGraph->RemoveNode(OldNode);
			}
		}

		// @todo fixme: below code doesn't work
		// because of SetAndCenterObject kicks in after new node is added
		// will need to disable that first
		TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();

		// Update the graph so that the node will be refreshed
		FocusedGraphEd->NotifyGraphChanged();
		// It's possible to leave invalid objects in the selection set if they get GC'd, so clear it out
		FocusedGraphEd->ClearSelectionSet();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetAnimBlueprint());
	}
}

void FAnimationBlueprintEditor::OnConvertToAimOffsetSimple()
{
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() > 0)
	{
		const FScopedTransaction Transaction( LOCTEXT("ConvertToSimpleAimOffset", "Convert to Simple Aim Offset") );

		for (auto NodeIter = SelectedNodes.CreateIterator(); NodeIter; ++NodeIter)
		{
			UAnimGraphNode_AimOffsetLookAt* OldNode = Cast<UAnimGraphNode_AimOffsetLookAt>(*NodeIter);

			// see if sequence player
			if (OldNode && OldNode->Node.GetBlendSpace())
			{
				// convert to sequence player
				UEdGraph* TargetGraph = OldNode->GetGraph();
				TargetGraph->Modify();
				OldNode->Modify();

				// create new player
				FGraphNodeCreator<UAnimGraphNode_RotationOffsetBlendSpace> NodeCreator(*TargetGraph);
				UAnimGraphNode_RotationOffsetBlendSpace* NewNode = NodeCreator.CreateNode();
				NewNode->Node.SetBlendSpace(OldNode->Node.GetBlendSpace());
				NodeCreator.Finalize();

				// get default data from old node to new node
				FEdGraphUtilities::CopyCommonState(OldNode, NewNode);

				CopyPinData(OldNode, NewNode, TEXT("X"));
				CopyPinData(OldNode, NewNode, TEXT("Y"));
				CopyPinData(OldNode, NewNode, TEXT("Pose"));
				CopyPinData(OldNode, NewNode, TEXT("BasePose"));

				// remove from selection and from graph
				NodeIter.RemoveCurrent();
				TargetGraph->RemoveNode(OldNode);
			}
		}

		// @todo fixme: below code doesn't work
		// because of SetAndCenterObject kicks in after new node is added
		// will need to disable that first
		TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
		// Update the graph so that the node will be refreshed
		FocusedGraphEd->NotifyGraphChanged();
		// It's possible to leave invalid objects in the selection set if they get GC'd, so clear it out
		FocusedGraphEd->ClearSelectionSet();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetAnimBlueprint());
	}
}

void FAnimationBlueprintEditor::OnConvertToAimOffsetGraph()
{
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() > 0)
	{
		const FScopedTransaction Transaction( LOCTEXT("ConvertToAimOffsetGraph", "Convert to Aim Offset Graph") );

		for (auto NodeIter = SelectedNodes.CreateIterator(); NodeIter; ++NodeIter)
		{
			UAnimGraphNode_RotationOffsetBlendSpace* OldNode = Cast<UAnimGraphNode_RotationOffsetBlendSpace>(*NodeIter);

			// see if sequence player
			if (OldNode && OldNode->Node.GetBlendSpace())
			{
				// convert to sequence player
				UEdGraph* TargetGraph = OldNode->GetGraph();
				TargetGraph->Modify();
				OldNode->Modify();

				// create new player
				FGraphNodeCreator<UAnimGraphNode_RotationOffsetBlendSpaceGraph> NodeCreator(*TargetGraph);
				UAnimGraphNode_RotationOffsetBlendSpaceGraph* NewNode = NodeCreator.CreateNode();
				if(OldNode->Node.GetGroupName() != NAME_None && OldNode->Node.GetGroupMethod() == EAnimSyncMethod::SyncGroup)
				{
					NewNode->SetSyncGroupName(OldNode->Node.GetGroupName());
				}
				NewNode->SetupFromAsset(OldNode->Node.GetBlendSpace(), false);
				NodeCreator.Finalize();

				// get default data from old node to new node
				FEdGraphUtilities::CopyCommonState(OldNode, NewNode);

				CopyPinData(OldNode, NewNode, TEXT("X"));
				CopyPinData(OldNode, NewNode, TEXT("Y"));
				CopyPinData(OldNode, NewNode, TEXT("Alpha"));
				CopyPinData(OldNode, NewNode, TEXT("Pose"));
				CopyPinData(OldNode, NewNode, TEXT("BasePose"));

				// remove from selection and from graph
				NodeIter.RemoveCurrent();
				TargetGraph->RemoveNode(OldNode);
			}
		}

		// @todo fixme: below code doesn't work
		// because of SetAndCenterObject kicks in after new node is added
		// will need to disable that first
		TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
		// Update the graph so that the node will be refreshed
		FocusedGraphEd->NotifyGraphChanged();
		// It's possible to leave invalid objects in the selection set if they get GC'd, so clear it out
		FocusedGraphEd->ClearSelectionSet();

		FBlueprintEditorUtils::MarkBlueprintAsStructurallyModified(GetAnimBlueprint());
	}
}

void FAnimationBlueprintEditor::OnOpenRelatedAsset()
{
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	
	EToolkitMode::Type Mode = EToolkitMode::Standalone;
	if (SelectedNodes.Num() > 0)
	{
		for (auto NodeIter = SelectedNodes.CreateIterator(); NodeIter; ++NodeIter)
		{
			if(UAnimGraphNode_Base* Node = Cast<UAnimGraphNode_Base>(*NodeIter))
			{
				UAnimationAsset* AnimAsset = Node->GetAnimationAsset();
				if(AnimAsset)
				{
					GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AnimAsset, Mode);
				}
			}
		}
	}
}

void FAnimationBlueprintEditor::OnCreateSelfTransition()
{
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() == 0)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("CreateSelfTransition", "Create Self Transition"));

	for (UObject* Node : SelectedNodes)
	{
		if (UAnimStateNode* StateNode = Cast<UAnimStateNode>(Node))
		{
			UEdGraph* TargetGraph = StateNode->GetGraph();
			TargetGraph->Modify();

			FGraphNodeCreator<UAnimStateAliasNode> AliasNodeCreator(*TargetGraph);
			UAnimStateAliasNode* AliasNode = AliasNodeCreator.CreateNode();
			TSharedPtr<INameValidatorInterface> NameValidator = AliasNode->MakeNameValidator();
			check(NameValidator.IsValid());
			FString NewName(TEXT("Self"));
			NameValidator->FindValidString(NewName);
			AliasNode->StateAliasName = NewName;
			AliasNodeCreator.Finalize();

			AliasNode->SetNodePosX(StateNode->GetNodePosX() + 200);
			AliasNode->SetNodePosY(StateNode->GetNodePosY() - 100);
			AliasNode->GetAliasedStates().Add(StateNode);
			
			FGraphNodeCreator<UAnimStateTransitionNode> TransitionNodeCreator(*TargetGraph);
			UAnimStateTransitionNode* TransitionNode = TransitionNodeCreator.CreateNode();
			TransitionNodeCreator.Finalize();

			// Self-transitions should always fall back to inertialization
			TransitionNode->bAllowInertializationForSelfTransitions = true;
			// Self transitions should always have a delay so they dont re-enter each frame (footgun prevention)
			TransitionNode->MinTimeBeforeReentry = 0.0f;
			TransitionNode->CreateConnections(AliasNode, StateNode);
		}
	}
}

FAnimNode_StateMachine* FAnimationBlueprintEditor::GetSelectedStateInstance(int32& OutSelectedState) const
{
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() != 1)
	{
		return nullptr;
	}

	UAnimStateNode* StateNode = Cast<UAnimStateNode>(*SelectedNodes.begin());
	if (StateNode == nullptr)
	{
		return nullptr;
	}

	UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(StateNode));
	if(AnimBlueprint == nullptr)
	{
		return nullptr;
	}

	UAnimInstance* ActiveObject = Cast<UAnimInstance>(AnimBlueprint->GetObjectBeingDebugged());
	UAnimBlueprintGeneratedClass* Class = AnimBlueprint->GetAnimBlueprintGeneratedClass();

	if ((ActiveObject == nullptr) || (Class == nullptr) || Class->GetAnimNodeProperties().Num() == 0)
	{
		return nullptr;
	}

	FStateMachineDebugData* DebugInfo = Class->GetAnimBlueprintDebugData().StateMachineDebugData.Find(StateNode->GetGraph());
	if (DebugInfo == nullptr)
	{
		return nullptr;
	}

	int32* StateIndexPtr = DebugInfo->NodeToStateIndex.Find(StateNode);
	if(StateIndexPtr == nullptr)
	{
		return nullptr;
	}

	if (!Class->BakedStateMachines.IsValidIndex(DebugInfo->MachineIndex))
	{
		return nullptr;
	}

	OutSelectedState = *StateIndexPtr;
	return const_cast<FAnimNode_StateMachine*>(ActiveObject->GetStateMachineInstanceFromName(Class->BakedStateMachines[DebugInfo->MachineIndex].MachineName));
}

bool FAnimationBlueprintEditor::CanSetActiveState() const
{
	int32 StateIndex = 0;
	return GetSelectedStateInstance(StateIndex) != nullptr;
}

void FAnimationBlueprintEditor::OnSetActiveState()
{
	int32 StateIndex = 0;
	FAnimNode_StateMachine* StateMachineInstance = GetSelectedStateInstance(StateIndex);
	if (StateMachineInstance == nullptr)
	{
		return;
	}

	StateMachineInstance->DebugForceState(StateIndex);
}

void FAnimationBlueprintEditor::OnReverseTransition()
{
	FGraphPanelSelectionSet SelectedNodes = GetSelectedNodes();
	if (SelectedNodes.Num() == 0)
	{
		return;
	}

	const FScopedTransaction Transaction(LOCTEXT("ReverseTransition", "Reverse Transition"));

	for (UObject* Node : SelectedNodes)
	{
		if (UAnimStateTransitionNode* TransitionNode = Cast<UAnimStateTransitionNode>(Node))
		{
			TransitionNode->Reverse();
		}
	}
}

bool FAnimationBlueprintEditor::CanRemovePosePin() const
{
	return true;
}

void FAnimationBlueprintEditor::RecompileAnimBlueprintIfDirty()
{
	if (UBlueprint* Blueprint = GetBlueprintObj())
	{
		if (!Blueprint->IsUpToDate())
		{
			Compile();
		}
	}
}

FName FAnimationBlueprintEditor::GetToolkitFName() const
{
	return FName("AnimationBlueprintEditor");
}

FName FAnimationBlueprintEditor::GetToolkitContextFName() const
{
	return FName("AnimationBlueprintEditor");
}

FText FAnimationBlueprintEditor::GetBaseToolkitName() const
{
	return LOCTEXT("AppLabel", "Animation Blueprint Editor");
}

FText FAnimationBlueprintEditor::GetToolkitToolTipText() const
{
	return FAssetEditorToolkit::GetToolTipTextForObject(GetBlueprintObj());
}

FString FAnimationBlueprintEditor::GetWorldCentricTabPrefix() const
{
	return LOCTEXT("WorldCentricTabPrefix", "Animation Blueprint Editor ").ToString();
}


FLinearColor FAnimationBlueprintEditor::GetWorldCentricTabColorScale() const
{
	return FLinearColor( 0.5f, 0.25f, 0.35f, 0.5f );
}

void FAnimationBlueprintEditor::InitToolMenuContext(FToolMenuContext& MenuContext)
{
	IAnimationBlueprintEditor::InitToolMenuContext(MenuContext);

	UAnimationBlueprintToolMenuContext* AnimationBlueprintToolMenuContext = NewObject<UAnimationBlueprintToolMenuContext>();
	AnimationBlueprintToolMenuContext->AnimationBlueprintEditor = SharedThis(this);
	MenuContext.AddObject(AnimationBlueprintToolMenuContext);

	UPersonaToolMenuContext* Context = NewObject<UPersonaToolMenuContext>();
	Context->SetToolkit(GetPersonaToolkit());

	MenuContext.AddObject(Context);
}

IAnimationSequenceBrowser* FAnimationBlueprintEditor::GetAssetBrowser() const
{
	return SequenceBrowser.Pin().Get();
}

void FAnimationBlueprintEditor::OnActiveTabChanged( TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated )
{
	if (!NewlyActivated.IsValid())
	{
		TArray<UObject*> ObjArray;
		Inspector->ShowDetailsForObjects(ObjArray);
	}
	else 
	{
		FBlueprintEditor::OnActiveTabChanged(PreviouslyActive, NewlyActivated);
	}
}

void FAnimationBlueprintEditor::SetPreviewMesh(USkeletalMesh* NewPreviewMesh)
{
	if(SkeletonTree.IsValid())
	{
		SkeletonTree->SetSkeletalMesh(NewPreviewMesh);
	}
}

void FAnimationBlueprintEditor::RefreshPreviewInstanceTrackCurves()
{
	// need to refresh the preview mesh
	UDebugSkelMeshComponent* PreviewMeshComponent = PersonaToolkit->GetPreviewMeshComponent();
	if(PreviewMeshComponent->PreviewInstance)
	{
		PreviewMeshComponent->PreviewInstance->RefreshCurveBoneControllers();
	}
}

void FAnimationBlueprintEditor::PostUndo(bool bSuccess)
{
	DocumentManager->CleanInvalidTabs();
	DocumentManager->RefreshAllTabs();

	FBlueprintEditor::PostUndo(bSuccess);

	// If we undid a node creation that caused us to clean up a tab/graph we need to refresh the UI state
	RefreshEditors();

	// PostUndo broadcast
	OnPostUndo.Broadcast();	

	RefreshPreviewInstanceTrackCurves();

	// clear up preview anim notify states
	// animnotify states are saved in AnimInstance
	// if those are undoed or redoed, they have to be 
	// cleared up, otherwise, they might have invalid data
	ClearupPreviewMeshAnimNotifyStates();

	OnPostCompile();
}

void FAnimationBlueprintEditor::ClearupPreviewMeshAnimNotifyStates()
{
	UDebugSkelMeshComponent* PreviewMeshComponent = PersonaToolkit->GetPreviewMeshComponent();
	if ( PreviewMeshComponent )
	{
		UAnimInstance* AnimInstanace = PreviewMeshComponent->GetAnimInstance();

		if (AnimInstanace)
		{
			// empty this because otherwise, it can have corrupted data
			// this will cause state to be interrupted, but that is better
			// than crashing
			AnimInstanace->ActiveAnimNotifyState.Empty();
		}
	}
}

UAnimInstance* FAnimationBlueprintEditor::GetPreviewInstance() const
{
	UDebugSkelMeshComponent* PreviewMeshComponent = PersonaToolkit->GetPreviewMeshComponent();
	if (PreviewMeshComponent->IsAnimBlueprintInstanced())
	{
		UAnimInstance* PreviewInstance = PreviewMeshComponent->GetAnimInstance();
		UAnimBlueprint* AnimBlueprint = GetAnimBlueprint();
		UAnimBlueprint* PreviewAnimBlueprint = AnimBlueprint->GetPreviewAnimationBlueprint();
		if (PreviewAnimBlueprint)
		{
			EPreviewAnimationBlueprintApplicationMethod ApplicationMethod = AnimBlueprint->GetPreviewAnimationBlueprintApplicationMethod();
			if(ApplicationMethod == EPreviewAnimationBlueprintApplicationMethod::LinkedLayers)
			{
				PreviewInstance = PreviewInstance->GetLinkedAnimLayerInstanceByClass(AnimBlueprint->GeneratedClass.Get());
			}
			else if(ApplicationMethod == EPreviewAnimationBlueprintApplicationMethod::LinkedAnimGraph)
			{
				PreviewInstance = PreviewInstance->GetLinkedAnimGraphInstanceByTag(AnimBlueprint->GetPreviewAnimationBlueprintTag());
			}
		}

		return PreviewInstance;
	}

	return nullptr;
}

void FAnimationBlueprintEditor::GetCustomDebugObjects(TArray<FCustomDebugObject>& DebugList) const
{
	UAnimInstance* PreviewInstance = GetPreviewInstance();
	if (PreviewInstance == nullptr)
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = PersonaToolkit->GetPreviewMeshComponent();
		PreviewInstance = PreviewMeshComponent->SavedAnimScriptInstance;
	}

	if (PreviewInstance)
	{
		new (DebugList) FCustomDebugObject(PreviewInstance, LOCTEXT("PreviewObjectLabel", "Preview Instance").ToString());
	}

	FAnimationBlueprintEditorModule& AnimationBlueprintEditorModule = FModuleManager::GetModuleChecked<FAnimationBlueprintEditorModule>("AnimationBlueprintEditor");
	AnimationBlueprintEditorModule.OnGetCustomDebugObjects().Broadcast(*this, DebugList);
}

FString FAnimationBlueprintEditor::GetCustomDebugObjectLabel(UObject* ObjectBeingDebugged) const
{
	UAnimInstance* PreviewInstance = GetPreviewInstance();
	if (PreviewInstance == nullptr)
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = PersonaToolkit->GetPreviewMeshComponent();
		PreviewInstance = PreviewMeshComponent->SavedAnimScriptInstance;
	}

	if (PreviewInstance == ObjectBeingDebugged)
	{
		return LOCTEXT("PreviewObjectLabel", "Preview Instance").ToString();
	}

	return FString();
}

void FAnimationBlueprintEditor::CreateDefaultTabContents(const TArray<UBlueprint*>& InBlueprints)
{
	FBlueprintEditor::CreateDefaultTabContents(InBlueprints);

	PreviewEditor = SNew(SAnimBlueprintPreviewPropertyEditor, SharedThis(this));
}

FGraphAppearanceInfo FAnimationBlueprintEditor::GetGraphAppearance(UEdGraph* InGraph) const
{
	FGraphAppearanceInfo AppearanceInfo = FBlueprintEditor::GetGraphAppearance(InGraph);

	if ( GetBlueprintObj()->IsA(UAnimBlueprint::StaticClass()) )
	{
		AppearanceInfo.CornerText = (GetDefault<UAnimationBlueprintEditorSettings>()->bShowGraphCornerText) ? LOCTEXT("AppearanceCornerText_Animation", "ANIMATION") : FText::GetEmpty();
	}

	return AppearanceInfo;
}

void FAnimationBlueprintEditor::ClearSelectedActor()
{
	GetPreviewScene()->ClearSelectedActor();
}

void FAnimationBlueprintEditor::ClearSelectedAnimGraphNodes()
{
	ReleaseAllManagedNodes();

	SelectedAnimGraphNodes.Empty();
}

void FAnimationBlueprintEditor::DeselectAll()
{
	if(SkeletonTree)
	{
		SkeletonTree->DeselectAll();
	}
	ClearSelectedActor();
	ClearSelectedAnimGraphNodes();
}

void FAnimationBlueprintEditor::PostRedo(bool bSuccess)
{
	DocumentManager->RefreshAllTabs();

	FBlueprintEditor::PostRedo(bSuccess);

	// PostUndo broadcast, OnPostRedo
	OnPostUndo.Broadcast();

	// clear up preview anim notify states
	// animnotify states are saved in AnimInstance
	// if those are undoed or redoed, they have to be 
	// cleared up, otherwise, they might have invalid data
	ClearupPreviewMeshAnimNotifyStates();

	// calls PostCompile to copy proper values between anim nodes
	OnPostCompile();
}

void FAnimationBlueprintEditor::UndoAction()
{
	GEditor->UndoTransaction();
}

void FAnimationBlueprintEditor::RedoAction()
{
	GEditor->RedoTransaction();
}

void FAnimationBlueprintEditor::NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged)
{
	FBlueprintEditor::NotifyPostChange(PropertyChangedEvent, PropertyThatChanged);

	// When you change properties on a node, call CopyNodeDataToPreviewNode to allow pushing those to preview instance, for live editing
	for (TWeakObjectPtr< class UAnimGraphNode_Base > CurrentAnimGraphNode : SelectedAnimGraphNodes)
	{
		if (UAnimGraphNode_Base* CurrentNode = CurrentAnimGraphNode.Get())
		{
			if (FAnimNode_Base* PreviewNode = FindAnimNode(CurrentNode))
			{
				CurrentNode->CopyNodeDataToPreviewNode(PreviewNode);
			}
		}
	}
}

void FAnimationBlueprintEditor::Tick(float DeltaTime)
{
	FBlueprintEditor::Tick(DeltaTime);

	GetPreviewScene()->InvalidateViews();
}

bool FAnimationBlueprintEditor::IsEditable(UEdGraph* InGraph) const
{
	bool bEditable = FBlueprintEditor::IsEditable(InGraph);

	if (InGraph)
	{
		bEditable &= (InGraph->GetTypedOuter<UBlueprint>() == GetBlueprintObj());
	}

	return bEditable;
}

FText FAnimationBlueprintEditor::GetGraphDecorationString(UEdGraph* InGraph) const
{
	if (!IsGraphInCurrentBlueprint(InGraph))
	{
		return LOCTEXT("PersonaExternalGraphDecoration", " External Graph Preview");
	}
	return FText::GetEmpty();
}

TStatId FAnimationBlueprintEditor::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(FAnimationBlueprintEditor, STATGROUP_Tickables);
}

void FAnimationBlueprintEditor::OnBlueprintPreCompile(UBlueprint* BlueprintToCompile)
{
	if (PersonaToolkit.IsValid())
	{
		UDebugSkelMeshComponent* PreviewMeshComponent = PersonaToolkit->GetPreviewMeshComponent();
		if(PreviewMeshComponent && PreviewMeshComponent->PreviewInstance)
		{
			// If we are compiling an anim notify state the class will soon be sanitized and 
			// if an anim instance is running a state when that happens it will likely
			// crash, so we end any states that are about to compile.
			UAnimPreviewInstance* Instance = PreviewMeshComponent->PreviewInstance;
			USkeletalMeshComponent* SkelMeshComp = Instance->GetSkelMeshComponent();

			for(int32 Idx = Instance->ActiveAnimNotifyState.Num() - 1 ; Idx >= 0 ; --Idx)
			{
				FAnimNotifyEvent& Event = Instance->ActiveAnimNotifyState[Idx];
				const FAnimNotifyEventReference& EventReference = Instance->ActiveAnimNotifyEventReference[Idx];
				if(Event.NotifyStateClass->GetClass() == BlueprintToCompile->GeneratedClass)
				{
					Event.NotifyStateClass->NotifyEnd(SkelMeshComp, Cast<UAnimSequenceBase>(Event.NotifyStateClass->GetOuter()), EventReference);
					check(Instance->ActiveAnimNotifyState.Num() == Instance->ActiveAnimNotifyEventReference.Num());
					Instance->ActiveAnimNotifyState.RemoveAt(Idx);
					Instance->ActiveAnimNotifyEventReference.RemoveAt(Idx);
				}
			}
		}
	}

	if(GetObjectsCurrentlyBeingEdited()->Num() > 0 && BlueprintToCompile == GetBlueprintObj())
	{
		// Grab the currently debugged object, so we can re-set it below in OnBlueprintPostCompile
		DebuggedMeshComponent = nullptr;

		UAnimInstance* CurrentDebugObject = Cast<UAnimInstance>(BlueprintToCompile->GetObjectBeingDebugged());
		if(CurrentDebugObject)
		{
			// Force close any asset editors that are using the AnimScriptInstance (such as the Property Matrix), the class will be garbage collected
			GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->CloseOtherEditors(CurrentDebugObject, nullptr);
			DebuggedMeshComponent = CurrentDebugObject->GetSkelMeshComponent();
		}
	}
}

void FAnimationBlueprintEditor::OnBlueprintPostCompile(UBlueprint* InBlueprint)
{
	if(InBlueprint == GetBlueprintObj())
	{
		if (DebuggedMeshComponent != nullptr)
		{
			if (DebuggedMeshComponent->GetAnimInstance() == nullptr)
			{
				// try reinitialize animation if it doesn't exist
				DebuggedMeshComponent->InitAnim(true);
			}

			// re-apply preview anim bp if needed
			UAnimBlueprint* AnimBlueprint = GetAnimBlueprint();
			UAnimBlueprint* PreviewAnimBlueprint = AnimBlueprint ? AnimBlueprint->GetPreviewAnimationBlueprint() : nullptr;
		
			if (PreviewAnimBlueprint)
			{
				PersonaToolkit->GetPreviewScene()->SetPreviewAnimationBlueprint(PreviewAnimBlueprint, AnimBlueprint);
			}

			if(UAnimInstance* NewInstance = DebuggedMeshComponent->GetAnimInstance())
			{
				if ((AnimBlueprint && NewInstance->IsA(AnimBlueprint->GeneratedClass)) || (PreviewAnimBlueprint && NewInstance->IsA(PreviewAnimBlueprint->GeneratedClass)))
				{
					PersonaUtils::SetObjectBeingDebugged(AnimBlueprint, NewInstance);
				}
			}
		}

		// reset the selected skeletal control nodes
		ClearSelectedAnimGraphNodes();

		// if the user manipulated Pin values directly from the node, then should copy updated values to the internal node to retain data consistency
		OnPostCompile();

		// We dont cache this persistently, only during a pre/post compile bracket
		DebuggedMeshComponent = nullptr;
	}

	// Make sure we re-enable ticking when we recompile (e.g. to fix an infinite loop that paused us)
	if (USkeletalMeshComponent* SkeletalMeshComponent = GetPreviewScene()->GetPreviewMeshComponent())
	{
		SkeletalMeshComponent->SetComponentTickEnabled(true);
	}
}

void FAnimationBlueprintEditor::OnBlueprintChangedImpl(UBlueprint* InBlueprint, bool bIsJustBeingCompiled /*= false*/)
{
	FBlueprintEditor::OnBlueprintChangedImpl(InBlueprint, bIsJustBeingCompiled);

	// calls PostCompile to copy proper values between anim nodes
	OnPostCompile();
}

void FAnimationBlueprintEditor::CreateEditorModeManager()
{
	EditorModeManager = MakeShareable(FModuleManager::LoadModuleChecked<FPersonaModule>("Persona").CreatePersonaEditorModeManager());
}

bool FAnimationBlueprintEditor::IsSectionVisible(NodeSectionID::Type InSectionID) const
{
	const UAnimBlueprintSettings* AnimBlueprintSettings = GetDefault<UAnimBlueprintSettings>();

	switch (InSectionID)
	{
	case NodeSectionID::GRAPH:
		return AnimBlueprintSettings->bAllowEventGraphs;
	case NodeSectionID::ANIMGRAPH:
	case NodeSectionID::ANIMLAYER:
	case NodeSectionID::FUNCTION:
	case NodeSectionID::FUNCTION_OVERRIDABLE:
	case NodeSectionID::INTERFACE:
		return true;
	case NodeSectionID::MACRO:
		return AnimBlueprintSettings->bAllowMacros;
	case NodeSectionID::VARIABLE:
		return true;
	case NodeSectionID::COMPONENT:
		return false;
	case NodeSectionID::DELEGATE:
		return AnimBlueprintSettings->bAllowDelegates;
	case NodeSectionID::USER_ENUM:
	case NodeSectionID::LOCAL_VARIABLE:
	case NodeSectionID::USER_STRUCT:
	case NodeSectionID::USER_SORTED:
		return true;
	default:
		break;
	}

	return true;
}

bool FAnimationBlueprintEditor::AreEventGraphsAllowed() const
{
	const UAnimBlueprintSettings* AnimBlueprintSettings = GetDefault<UAnimBlueprintSettings>();
	return AnimBlueprintSettings->bAllowEventGraphs;
}

bool FAnimationBlueprintEditor::AreMacrosAllowed() const
{
	const UAnimBlueprintSettings* AnimBlueprintSettings = GetDefault<UAnimBlueprintSettings>();
	return AnimBlueprintSettings->bAllowMacros;
}

bool FAnimationBlueprintEditor::AreDelegatesAllowed() const
{
	const UAnimBlueprintSettings* AnimBlueprintSettings = GetDefault<UAnimBlueprintSettings>();
	return AnimBlueprintSettings->bAllowDelegates;
}

void FAnimationBlueprintEditor::OnCreateComment()
{
	TSharedPtr<SGraphEditor> GraphEditor = FocusedGraphEdPtr.Pin();
	if (GraphEditor.IsValid())
	{
		if (UEdGraph* Graph = GraphEditor->GetCurrentGraph())
		{
			FEdGraphSchemaAction_K2AddComment CommentAction;
			CommentAction.PerformAction(Graph, nullptr, GraphEditor->GetPasteLocation2f());
		}
	}
}

void FAnimationBlueprintEditor::JumpToHyperlink(const UObject* ObjectReference, bool bRequestRename)
{
	if(const UBlendSpaceGraph* BlendSpaceGraph = Cast<UBlendSpaceGraph>(ObjectReference))
	{
		TSharedRef<FTabPayload_BlendSpaceGraph> Payload = FTabPayload_BlendSpaceGraph::Make(BlendSpaceGraph);
		DocumentManager->OpenDocument(Payload, FDocumentTracker::OpenNewDocument);
	}
	else
	{
		FBlueprintEditor::JumpToHyperlink(ObjectReference, bRequestRename);
	}
}

TSharedRef<IPersonaPreviewScene> FAnimationBlueprintEditor::GetPreviewScene() const
{ 
	return PersonaToolkit->GetPreviewScene(); 
}

void FAnimationBlueprintEditor::HandleObjectsSelected(const TArray<UObject*>& InObjects)
{
	SetDetailObjects(InObjects);
}

void FAnimationBlueprintEditor::HandleObjectSelected(UObject* InObject)
{
	SetDetailObject(InObject);
}

void FAnimationBlueprintEditor::HandleSelectionChanged(const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type InSelectInfo)
{
	TArray<UObject*> Objects;
	Algo::TransformIf(InSelectedItems, Objects, [](const TSharedPtr<ISkeletonTreeItem>& InItem) { return InItem->GetObject() != nullptr; }, [](const TSharedPtr<ISkeletonTreeItem>& InItem) { return InItem->GetObject(); });
	SetDetailObjects(Objects);
}

UObject* FAnimationBlueprintEditor::HandleGetObject()
{
	return GetEditingObject();
}

void FAnimationBlueprintEditor::HandleOpenNewAsset(UObject* InNewAsset)
{
	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(InNewAsset);
}

void FAnimationBlueprintEditor::AddReferencedObjects( FReferenceCollector& Collector )
{
	Collector.AddReferencedObject( EditorOptions );
}

FAnimNode_Base* FAnimationBlueprintEditor::FindAnimNode(UAnimGraphNode_Base* AnimGraphNode) const
{
	FAnimNode_Base* AnimNode = nullptr;
	if (AnimGraphNode)
	{
		USkeletalMeshComponent* SkeletalMeshComponentToUse = nullptr;
		if(UAnimInstance* AnimInstance = Cast<UAnimInstance>(GetAnimBlueprint()->GetObjectBeingDebugged()))
		{
			SkeletalMeshComponentToUse = AnimInstance->GetSkelMeshComponent();
		}
		else
		{
			SkeletalMeshComponentToUse = GetPreviewScene()->GetPreviewMeshComponent();
		}

		if (SkeletalMeshComponentToUse != nullptr && SkeletalMeshComponentToUse->GetAnimInstance() != nullptr)
		{
			AnimNode = AnimGraphNode->FindDebugAnimNode(SkeletalMeshComponentToUse);
		}
	}

	return AnimNode;
}

void FAnimationBlueprintEditor::OnSelectedNodesChangedImpl(const TSet<class UObject*>& NewSelection)
{
	FBlueprintEditor::OnSelectedNodesChangedImpl(NewSelection);

	IPersonaEditorModeManager* const PersonaEditorModeManager = static_cast<IPersonaEditorModeManager*>(&GetEditorModeManager());

	if (PersonaEditorModeManager)
	{
		// Update the list of selected nodes, being careful to maintain the order of the list as this is an important requirement of the UI.

		using FSelectedNodePtr = TWeakObjectPtr< class UAnimGraphNode_Base >;

		TArray< FSelectedNodePtr > AddSelection;	// Nodes that should be added to the current selection.
		TArray< FSelectedNodePtr > RemSelection;	// Nodes that should be removed from the current selection.

		// Compare the set of nodes in 'NewSelection' with the list of previously selected nodes to identify nodes that should be added / removed from the selection.
		{
			TArray< FSelectedNodePtr > OldSelectionSorted(SelectedAnimGraphNodes);
			TArray< FSelectedNodePtr > NewSelectionSorted;

			for (UObject* NewSelectedObject : NewSelection)
			{
				if (UAnimGraphNode_Base* NewSelectedAnimGraphNode = Cast<UAnimGraphNode_Base>(NewSelectedObject))
				{
					NewSelectionSorted.Add(NewSelectedAnimGraphNode);
				}
			}

			auto SortPredicate = [](const FSelectedNodePtr& Lhs, const FSelectedNodePtr& Rhs) { return Lhs.Get() < Rhs.Get();  };

			OldSelectionSorted.Sort(SortPredicate);
			NewSelectionSorted.Sort(SortPredicate);

			SortedContainerDifference(OldSelectionSorted, NewSelectionSorted, AddSelection, RemSelection, SortPredicate);
		}
		
		ReleaseAllManagedNodes(); // Register de-selection with all the previously selected nodes.


		// Remove all the nodes that are no longer selected.
		for (FSelectedNodePtr CurrentAnimGraphNode : RemSelection)
		{
			SelectedAnimGraphNodes.Remove(CurrentAnimGraphNode);
		}

		// Add all the newly selected nodes.
		for (FSelectedNodePtr CurrentAnimGraphNode : AddSelection)
		{
			SelectedAnimGraphNodes.Add(CurrentAnimGraphNode);
		}

		AcquireAllManagedNodes(); // Register re-selection with all the currently selected nodes.
	}

	bSelectRegularNode = false;
	for (FGraphPanelSelectionSet::TConstIterator It(NewSelection); It; ++It)
	{
		UEdGraphNode_Comment* SeqNode = Cast<UEdGraphNode_Comment>(*It);
		UAnimStateNodeBase* AnimGraphNodeBase = Cast<UAnimStateNodeBase>(*It);
		UAnimStateEntryNode* AnimStateEntryNode = Cast<UAnimStateEntryNode>(*It);
		if (!SeqNode && !AnimGraphNodeBase && !AnimStateEntryNode)
		{
			bSelectRegularNode = true;
			break;
		}
	}

	if (bHideUnrelatedNodes && !bLockNodeFadeState)
	{
		ResetAllNodesUnrelatedStates();

		if ( bSelectRegularNode )
		{
			HideUnrelatedNodes();
		}
	}

	if (GetDefault<UAnimationBlueprintEditorSettings>()->bPoseWatchSelectedNodes)
	{
		HandlePoseWatchSelectedNodes();
	}
}

void FAnimationBlueprintEditor::HandlePoseWatchSelectedNodes()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		ReleaseAllManagedNodes(); // Register de-selection with all the previously selected nodes.

		UAnimBlueprint* AnimBP = GetAnimBlueprint();
		TArray<UEdGraphNode*> AllNodes = FocusedGraphEd->GetCurrentGraph()->Nodes;

		FGraphPanelSelectionSet SelectionNodes = GetSelectedNodes();

		for (UEdGraphNode* Node : AllNodes)
		{
			UAnimGraphNode_Base* GraphNode = Cast<UAnimGraphNode_Base>(Node);
			UPoseWatch* PoseWatch = AnimationEditorUtils::FindPoseWatchForNode(GraphNode, AnimBP);
			if (GraphNode)
			{
				if (SelectionNodes.Contains(Node))
				{
					if (!PoseWatch && GraphNode->IsPoseWatchable())
					{
						PoseWatch = AnimationEditorUtils::MakePoseWatchForNode(AnimBP, GraphNode);
						PoseWatch->SetShouldDeleteOnDeselect(true);
					}
				}
				else
				{
					if (PoseWatch && PoseWatch->GetShouldDeleteOnDeselect())
					{
						AnimationEditorUtils::RemovePoseWatch(PoseWatch, AnimBP);
					}
				}
			}
		}

		AcquireAllManagedNodes(); // Register re-selection with all the currently selected nodes.
	}
}

void FAnimationBlueprintEditor::RemoveAllSelectionPoseWatches()
{
	TSharedPtr<SGraphEditor> FocusedGraphEd = FocusedGraphEdPtr.Pin();
	if (FocusedGraphEd.IsValid())
	{
		ReleaseAllManagedNodes(); // Register de-selection with all the previously selected nodes.

		UAnimBlueprint* AnimBP = GetAnimBlueprint();
		TArray<UEdGraphNode*> AllNodes = FocusedGraphEd->GetCurrentGraph()->Nodes;

		for (UEdGraphNode* Node : AllNodes)
		{
			UAnimGraphNode_Base* GraphNode = Cast<UAnimGraphNode_Base>(Node);
			if (GraphNode)
			{
				UPoseWatch* PoseWatch = AnimationEditorUtils::FindPoseWatchForNode(GraphNode, AnimBP);
				if (PoseWatch && PoseWatch->GetShouldDeleteOnDeselect())
				{
					AnimationEditorUtils::RemovePoseWatch(PoseWatch, AnimBP);
				}
			}
		}

		AcquireAllManagedNodes(); // Register re-selection with all the currently selected nodes.
	}
}
void FAnimationBlueprintEditor::OnPostCompile()
{
	// act as if we have re-selected, so internal pointers are updated
	if (CurrentUISelection == FBlueprintEditor::SelectionState_Graph)
	{
		FGraphPanelSelectionSet SelectionSet = GetSelectedNodes();
		OnSelectedNodesChangedImpl(SelectionSet);
		FocusInspectorOnGraphSelection(SelectionSet, /*bForceRefresh=*/ true);
	}

	// if the user manipulated Pin values directly from the node, then should copy updated values to the internal node to retain data consistency
	UEdGraph* FocusedGraph = GetFocusedGraph();
	if (FocusedGraph)
	{
		// find UAnimGraphNode_Base
		for (UEdGraphNode* Node : FocusedGraph->Nodes)
		{
			UAnimGraphNode_Base* AnimGraphNode = Cast<UAnimGraphNode_Base>(Node);
			if (AnimGraphNode)
			{
				FAnimNode_Base* AnimNode = FindAnimNode(AnimGraphNode);
				if (AnimNode)
				{
					AnimGraphNode->CopyNodeDataToPreviewNode(AnimNode);
				}
			}
		}
	}
}

void FAnimationBlueprintEditor::HandlePinDefaultValueChanged(UEdGraphPin* InPinThatChanged)
{
	UAnimGraphNode_Base* AnimGraphNode = Cast<UAnimGraphNode_Base>(InPinThatChanged->GetOwningNode());
	if (AnimGraphNode)
	{
		FAnimNode_Base* AnimNode = FindAnimNode(AnimGraphNode);
		if (AnimNode)
		{
			AnimGraphNode->CopyNodeDataToPreviewNode(AnimNode);
		}
	}
}

void FAnimationBlueprintEditor::HandleSetObjectBeingDebugged(UObject* InObject)
{
	FBlueprintEditor::HandleSetObjectBeingDebugged(InObject);

	// act as if we have re-selected, so internal pointers are updated
	if (CurrentUISelection == FBlueprintEditor::SelectionState_Graph)
	{
		FGraphPanelSelectionSet SelectionSet = GetSelectedNodes();
		OnSelectedNodesChangedImpl(SelectionSet);
	}
	
	if (UAnimInstance* AnimInstance = Cast<UAnimInstance>(InObject))
	{
		USkeletalMeshComponent* SkeletalMeshComponent = AnimInstance->GetSkelMeshComponent();
		if (SkeletalMeshComponent)
		{
			// If we are selecting the preview instance, reset us back to 'normal'
			if (InObject->GetWorld()->IsPreviewWorld())
			{
				GetPreviewScene()->ShowDefaultMode();
				if(GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance)
				{
					GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance->SetDebugSkeletalMeshComponent(nullptr);
				}
				if (GetPreviewScene()->GetPreviewMeshComponent()->bTrackAttachedInstanceLOD)
				{
					GetPreviewScene()->GetPreviewMeshComponent()->bTrackAttachedInstanceLOD = false;
					GetPreviewScene()->GetPreviewMeshComponent()->SetForcedLOD(0);
				}
			}
			else
			{
				// Otherwise set us to display the debugged instance via copy-pose
				GetPreviewScene()->GetPreviewMeshComponent()->EnablePreview(true, nullptr);
				if (GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance)
				{
					GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance->SetDebugSkeletalMeshComponent(SkeletalMeshComponent);
					GetPreviewScene()->GetPreviewMeshComponent()->bTrackAttachedInstanceLOD = true;
				}
			}
		}
	}
	else
	{
		// Clear the copy-pose component and set us back to 'normal'
		GetPreviewScene()->ShowDefaultMode();
		if(GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance)
		{
			GetPreviewScene()->GetPreviewMeshComponent()->PreviewInstance->SetDebugSkeletalMeshComponent(nullptr);
		}
		
		if (GetPreviewScene()->GetPreviewMeshComponent()->bTrackAttachedInstanceLOD)
		{
			GetPreviewScene()->GetPreviewMeshComponent()->bTrackAttachedInstanceLOD = false;
			GetPreviewScene()->GetPreviewMeshComponent()->SetForcedLOD(0);
		}
	}
}

void FAnimationBlueprintEditor::HandlePreviewMeshChanged(USkeletalMesh* OldPreviewMesh, USkeletalMesh* NewPreviewMesh)
{
	UObject* Object = GetBlueprintObj()->GetObjectBeingDebugged();
	if(Object)
	{
		HandleSetObjectBeingDebugged(Object);
	}
}

void FAnimationBlueprintEditor::HandleViewportCreated(const TSharedRef<IPersonaViewport>& InPersonaViewport)
{
	auto GetCompilationStateText = [this]()
	{
		if (UBlueprint* Blueprint = GetBlueprintObj())
		{
			switch (Blueprint->Status)
			{
			case BS_UpToDate:
			case BS_UpToDateWithWarnings:
				// Fall thru and return empty string
				break;
			case BS_Dirty:
				return LOCTEXT("AnimBP_Dirty", "Preview out of date");
			case BS_Error:
				return LOCTEXT("AnimBP_CompileError", "Compile Error");
			default:
				return LOCTEXT("AnimBP_UnknownStatus", "Unknown Status");
			}
		}

		return FText::GetEmpty();
	};

	auto GetCompilationStateVisibility = [this]()
	{
		if (UBlueprint* Blueprint = GetBlueprintObj())
		{
			const bool bUpToDate = (Blueprint->Status == BS_UpToDate) || (Blueprint->Status == BS_UpToDateWithWarnings);
			return bUpToDate ? EVisibility::Collapsed : EVisibility::Visible;
		}

		return EVisibility::Collapsed;
	};

	auto GetCompileButtonVisibility = [this]()
	{
		if (UBlueprint* Blueprint = GetBlueprintObj())
		{
			return (Blueprint->Status == BS_Dirty) ? EVisibility::Visible : EVisibility::Collapsed;
		}

		return EVisibility::Collapsed;
	};

	auto CompileBlueprint = [this]()
	{
		if (UBlueprint* Blueprint = GetBlueprintObj())
		{
			if (!Blueprint->IsUpToDate())
			{
				Compile();
			}
		}

		return FReply::Handled();
	};

	auto GetErrorSeverity = [this]()
	{
		if (UBlueprint* Blueprint = GetBlueprintObj())
		{
			return (Blueprint->Status == BS_Error) ? EMessageSeverity::Error : EMessageSeverity::Warning;
		}

		return EMessageSeverity::Warning;
	};

	auto GetIcon = [this]()
	{
		if (UBlueprint* Blueprint = GetBlueprintObj())
		{
			return (Blueprint->Status == BS_Error) ? FEditorFontGlyphs::Exclamation_Triangle : FEditorFontGlyphs::Eye;
		}

		return FEditorFontGlyphs::Eye;
	};

	InPersonaViewport->AddNotification(MakeAttributeLambda(GetErrorSeverity),
		false,
		SNew(SHorizontalBox)
		.Visibility_Lambda(GetCompilationStateVisibility)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(4.0f, 4.0f)
		[
			SNew(SHorizontalBox)
			.ToolTipText_Lambda(GetCompilationStateText)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
				.Text_Lambda(GetIcon)
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text_Lambda(GetCompilationStateText)
				.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(2.0f, 0.0f)
		[
			SNew(SButton)
			.ForegroundColor(FSlateColor::UseForeground())
			.ButtonStyle(FAppStyle::Get(), "FlatButton.Success")
			.Visibility_Lambda(GetCompileButtonVisibility)
			.ToolTipText(LOCTEXT("AnimBPViewportCompileButtonToolTip", "Compile this Animation Blueprint to update the preview to reflect any recent changes."))
			.OnClicked_Lambda(CompileBlueprint)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(0.0f, 0.0f, 4.0f, 0.0f)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
					.Text(FEditorFontGlyphs::Cog)
				]
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
					.Text(LOCTEXT("AnimBPViewportCompileButtonLabel", "Compile"))
				]
			]
		],
		FPersonaViewportNotificationOptions(TAttribute<EVisibility>::Create(GetCompilationStateVisibility))
	);

	auto GetInfiniteLoopVisibility = [this]()
	{
		if (USkeletalMeshComponent* SkeletalMeshComponent = GetPreviewScene()->GetPreviewMeshComponent())
		{
			return SkeletalMeshComponent->IsComponentTickEnabled() ? EVisibility::Collapsed : EVisibility::Visible;
		}
		return EVisibility::Collapsed;
	};

	InPersonaViewport->AddNotification(EMessageSeverity::Error,
		false,
		SNew(SHorizontalBox)
		.Visibility_Lambda(GetInfiniteLoopVisibility)
		+SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.Padding(4.0f, 4.0f)
		[
			SNew(SHorizontalBox)
			.ToolTipText_Lambda(GetCompilationStateText)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0.0f, 0.0f, 4.0f, 0.0f)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.9"))
				.Text(FEditorFontGlyphs::Exclamation_Triangle)
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("InfiniteLoopDetected", "Infinite Loop Detected"))
				.TextStyle(FAppStyle::Get(), "AnimViewport.MessageText")
			]
		],
		FPersonaViewportNotificationOptions(TAttribute<EVisibility>::Create(GetInfiniteLoopVisibility))
	);
}

void FAnimationBlueprintEditor::LoadEditorSettings()
{
	EditorOptions = NewObject<UAnimationBlueprintEditorOptions>();

	if (EditorOptions->bHideUnrelatedNodes)
	{
		ToggleHideUnrelatedNodes();
	}
}

void FAnimationBlueprintEditor::SaveEditorSettings()
{
	if ( EditorOptions )
	{
		EditorOptions->bHideUnrelatedNodes = bHideUnrelatedNodes;
		EditorOptions->SaveConfig();
	}
}

void FAnimationBlueprintEditor::HandlePreviewAnimBlueprintCompiled(UBlueprint* InBlueprint)
{
	UAnimBlueprint* AnimBlueprint = GetAnimBlueprint();
	UAnimBlueprint* PreviewAnimBlueprint = AnimBlueprint->GetPreviewAnimationBlueprint();
	if (PreviewAnimBlueprint)
	{
		GetPreviewScene()->SetPreviewAnimationBlueprint(PreviewAnimBlueprint, AnimBlueprint);
	}
}

void FAnimationBlueprintEditor::HandleAnimationSequenceBrowserCreated(const TSharedRef<IAnimationSequenceBrowser>& InSequenceBrowser)
{
	SequenceBrowser = InSequenceBrowser;
}

void FAnimationBlueprintEditor::HandleScriptException(const UObject* InObject, const FFrame& InFrame, const FBlueprintExceptionInfo& InInfo)
{
	// If the object is an anim instance in our preview world and is infinitely looping, disable ticking (renabled on recompilation)
	if (InInfo.GetType() == EBlueprintExceptionType::InfiniteLoop)
	{
		if (InObject && InObject->IsA<UAnimInstance>())
		{
			UWorld* ObjectWorld = InObject->GetWorld();
			TSharedRef<IPersonaPreviewScene> ThisPreviewScene = GetPreviewScene();
			if (ObjectWorld == ThisPreviewScene->GetWorld())
			{
				if (USkeletalMeshComponent* SkeletalMeshComponent = ThisPreviewScene->GetPreviewMeshComponent())
				{
					SkeletalMeshComponent->SetComponentTickEnabled(false);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE

