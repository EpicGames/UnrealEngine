// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditor.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphPin.h"
#include "GraphEditor.h"
#include "HAL/Platform.h"
#include "IAnimationBlueprintEditor.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Stats/Stats.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/IToolkit.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtrTemplates.h"

class FExtender;
class FReferenceCollector;
class FUICommandList;
class IAnimationSequenceBrowser;
class IPersonaViewport;
class ISkeletonTreeItem;
class SDockTab;
class SWidget;
class UAnimBlueprint;
class UAnimInstance;
class UAnimationBlueprintEditorOptions;
class UAnimationBlueprintEditorSettings;
class UBlueprint;
class UEdGraph;
class UObject;
class USkeletalMesh;
class USkeletalMeshComponent;
struct FToolMenuContext;
struct FFrame;
struct FBlueprintExceptionInfo;
struct FAnimNode_StateMachine;

struct FAnimationBlueprintEditorModes
{
	// Mode constants
	static const FName AnimationBlueprintEditorMode;
	static const FName AnimationBlueprintInterfaceEditorMode;
	static const FName AnimationBlueprintTemplateEditorMode;

	static FText GetLocalizedMode(const FName InMode)
	{
		static TMap< FName, FText > LocModes;

		if (LocModes.Num() == 0)
		{
			LocModes.Add(AnimationBlueprintEditorMode, NSLOCTEXT("AnimationBlueprintEditorModes", "AnimationBlueprintEditorMode", "Animation Blueprint"));
			LocModes.Add(AnimationBlueprintInterfaceEditorMode, NSLOCTEXT("AnimationBlueprintEditorModes", "AnimationBlueprintInterface EditorMode", "Animation Blueprint Interface"));
			LocModes.Add(AnimationBlueprintTemplateEditorMode, NSLOCTEXT("AnimationBlueprintEditorModes", "AnimationBlueprintTemplate EditorMode", "Animation Blueprint Template"));
		}

		check(InMode != NAME_None);
		const FText* OutDesc = LocModes.Find(InMode);
		check(OutDesc);
		return *OutDesc;
	}
private:
	FAnimationBlueprintEditorModes() {}
};

namespace AnimationBlueprintEditorTabs
{
	extern const FName DetailsTab;
	extern const FName SkeletonTreeTab;
	extern const FName ViewportTab;
	extern const FName AdvancedPreviewTab;
	extern const FName AssetBrowserTab;
	extern const FName AnimBlueprintPreviewEditorTab;
	extern const FName AssetOverridesTab;
	extern const FName SlotNamesTab;
	extern const FName CurveNamesTab;
	extern const FName PoseWatchTab;
	extern const FName FindReplaceTab;
};

/**
 * Animation Blueprint asset editor (extends Blueprint editor)
 */
class FAnimationBlueprintEditor : public IAnimationBlueprintEditor
{
	friend class FAnimationBlueprintEditorMode;
	friend class FAnimationBlueprintTemplateEditorMode;

public:
	/**
	 * Edits the specified character asset(s)
	 *
	 * @param	Mode					Mode that this editor should operate in
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	InitSkeleton			The skeleton to edit.  If specified, Blueprint must be NULL.
	 * @param	InitAnimBlueprint		The blueprint object to start editing.  If specified, Skeleton and AnimationAsset must be NULL.
	 * @param	InitAnimationAsset		The animation asset to edit.  If specified, Blueprint must be NULL.
	 * @param	InitMesh				The mesh asset to edit.  If specified, Blueprint must be NULL.	 
	 */
	void InitAnimationBlueprintEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UAnimBlueprint* InAnimBlueprint);

public:
	FAnimationBlueprintEditor();

	virtual ~FAnimationBlueprintEditor();

	/** Update the inspector that displays information about the current selection*/
	void SetDetailObjects(const TArray<UObject*>& InObjects);
	void SetDetailObject(UObject* Obj);

	/** IAnimationBlueprintEditor interface */
	virtual const FEdGraphPinType& GetLastGraphPinTypeUsed() const override { return LastGraphPinType; }
	virtual void SetLastGraphPinTypeUsed(const FEdGraphPinType& InType) override { LastGraphPinType = InType; }
	virtual IAnimationSequenceBrowser* GetAssetBrowser() const override;
	virtual UAnimInstance* GetPreviewInstance() const override;

	/** IHasPersonaToolkit interface */
	virtual TSharedRef<class IPersonaToolkit> GetPersonaToolkit() const { return PersonaToolkit.ToSharedRef(); }

	/** FBlueprintEditor interface */
	virtual void OnActiveTabChanged(TSharedPtr<SDockTab> PreviouslyActive, TSharedPtr<SDockTab> NewlyActivated) override;
	virtual void OnSelectedNodesChangedImpl(const TSet<class UObject*>& NewSelection) override;
	virtual void HandleSetObjectBeingDebugged(UObject* InObject) override;

	// Gets the Anim Blueprint being edited/viewed by this Persona instance
	UAnimBlueprint* GetAnimBlueprint() const;

	// Sets the current preview mesh
	void SetPreviewMesh(USkeletalMesh* NewPreviewMesh);

	/** Clears the selected actor */
	void ClearSelectedActor();

	/** Clears the selected anim graph nodes */
	void ClearSelectedAnimGraphNodes();

	/** Clears the selection (both sockets and bones). Also broadcasts this */
	void DeselectAll();

	/** Returns the editors preview scene */
	TSharedRef<class IPersonaPreviewScene> GetPreviewScene() const;

	/** Handle general object selection */
	void HandleObjectsSelected(const TArray<UObject*>& InObjects);
	void HandleObjectSelected(UObject* InObject);
	void HandleSelectionChanged(const TArrayView<TSharedPtr<ISkeletonTreeItem>>& InSelectedItems, ESelectInfo::Type InSelectInfo);

	/** Get the object to be displayed in the asset properties */
	UObject* HandleGetObject();
	
	//~ Begin FGCObject Interface
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	//~ End FGCObject Interface

	/** Handle opening a new asset from the asset browser */
	void HandleOpenNewAsset(UObject* InNewAsset);

public:
	//~ Begin IToolkit Interface
	virtual FName GetToolkitFName() const override;
	virtual FName GetToolkitContextFName() const override;
	virtual FText GetBaseToolkitName() const override;
	virtual FText GetToolkitToolTipText() const override;
	virtual FString GetWorldCentricTabPrefix() const override;
	virtual FLinearColor GetWorldCentricTabColorScale() const override;	
	virtual void InitToolMenuContext(FToolMenuContext& MenuContext) override;
	//~ End IToolkit Interface

	/** @return the documentation location for this editor */
	virtual FString GetDocumentationLink() const override
	{
		return FString(TEXT("Engine/Animation/Persona"));
	}
	
	/** Returns a pointer to the Blueprint object we are currently editing, as long as we are editing exactly one */
	virtual UBlueprint* GetBlueprintObj() const override;

	//~ Begin FTickableEditorObject Interface
	virtual void Tick(float DeltaTime) override;
	virtual TStatId GetStatId() const override;
	//~ End FTickableEditorObject Interface

	//~ Begin FBlueprintEditor Interface
	virtual void JumpToHyperlink(const UObject* ObjectReference, bool bRequestRename) override;
	//~ End FBlueprintEditor Interface

	TSharedRef<SWidget> GetPreviewEditor() { return PreviewEditor.ToSharedRef(); }
	/** Refresh Preview Instance Track Curves **/
	void RefreshPreviewInstanceTrackCurves();

	void RecompileAnimBlueprintIfDirty();

	/** Get the skeleton tree this Persona editor is hosting */
	TSharedPtr<class ISkeletonTree> GetSkeletonTree() const { return SkeletonTree; }

	/** Make this available to allow us to create title bar widgets for other container types - e.g. blendspaces */
	using FBlueprintEditor::CreateGraphTitleBarWidget;

protected:
	//~ Begin FBlueprintEditor Interface
	//virtual void CreateDefaultToolbar() override;
	virtual void CreateDefaultCommands() override;
	virtual void OnCreateGraphEditorCommands(TSharedPtr<FUICommandList> GraphEditorCommandsList);
	virtual void OnGraphEditorFocused(const TSharedRef<class SGraphEditor>& InGraphEditor) override;
	virtual void OnGraphEditorBackgrounded(const TSharedRef<SGraphEditor>& InGraphEditor) override;
	virtual bool IsInAScriptingMode() const override { return true; }
	virtual void GetCustomDebugObjects(TArray<FCustomDebugObject>& DebugList) const override;
	virtual FString GetCustomDebugObjectLabel(UObject* ObjectBeingDebugged) const override;
	virtual void CreateDefaultTabContents(const TArray<UBlueprint*>& InBlueprints) override;
	virtual FGraphAppearanceInfo GetGraphAppearance(class UEdGraph* InGraph) const override;
	virtual bool IsEditable(UEdGraph* InGraph) const override;
	virtual FText GetGraphDecorationString(UEdGraph* InGraph) const override;
	virtual void OnBlueprintChangedImpl(UBlueprint* InBlueprint, bool bIsJustBeingCompiled = false) override;
	virtual void CreateEditorModeManager() override;
	virtual bool IsSectionVisible(NodeSectionID::Type InSectionID) const override;
	virtual bool AreEventGraphsAllowed() const override;
	virtual bool AreMacrosAllowed() const override;
	virtual bool AreDelegatesAllowed() const override;
	virtual void OnCreateComment() override;
	//~ End FBlueprintEditor Interface

	//~ Begin FEditorUndoClient Interface
	virtual void PostUndo(bool bSuccess) override;
	virtual void PostRedo(bool bSuccess) override;
	// End of FEditorUndoClient

	//~ Begin FNotifyHook Interface
	virtual void NotifyPostChange(const FPropertyChangedEvent& PropertyChangedEvent, FProperty* PropertyThatChanged) override;
	//~ End FNotifyHook Interface

	// Toggle pose watch on selected nodes
	bool CanTogglePoseWatch();
	void OnTogglePoseWatch();

	// Hide unbound pins on selected nodes 
	bool CanHideUnboundPropertyPins();
	void OnHideUnboundPropertyPins();

	void BindCommands();

protected:

	/** Clear up Preview Mesh's AnimNotifyStates. Called when undo or redo**/
	void ClearupPreviewMeshAnimNotifyStates();

public:
	/** Viewport widget */
	TWeakPtr<class IPersonaViewport> Viewport;

	/** holding this pointer to refresh persona mesh detials tab when LOD is changed **/
	class IDetailLayoutBuilder* PersonaMeshDetailLayout;

public:

	// Called after an undo is performed to give child widgets a chance to refresh
	typedef FSimpleMulticastDelegate::FDelegate FOnPostUndo;

	/** Registers a delegate to be called after an Undo operation */
	void RegisterOnPostUndo(const FOnPostUndo& Delegate)
	{
		OnPostUndo.Add(Delegate);
	}

	/** Unregisters a delegate to be called after an Undo operation */
	void UnregisterOnPostUndo(SWidget* Widget)
	{
		OnPostUndo.RemoveAll(Widget);
	}

	/** Delegate called after an undo operation for child widgets to refresh */
	FSimpleMulticastDelegate OnPostUndo;

protected:
	/** Undo Action**/
	void UndoAction();
	/** Redo Action **/
	void RedoAction();

private:

	/** Extend menu */
	void ExtendMenu();

	/** Register menus */
	void RegisterMenus();

	/** Extend toolbar */
	void ExtendToolbar();

	/** Get the anim BP editor referenced by the supplied tool menu context */
	static TSharedPtr<FAnimationBlueprintEditor> GetAnimationBlueprintEditor(const FToolMenuContext& InMenuContext);

	/** Called immediately prior to a blueprint compilation */
	void OnBlueprintPreCompile(UBlueprint* BlueprintToCompile);

	/** Called immediately after to a blueprint compilation */
	void OnBlueprintPostCompile(UBlueprint* InBlueprint);

	/** Called post compile to copy node data */
	void OnPostCompile();

	/** Call OnNodeSelected for each selected node **/
	void NotifyAllNodesOnSelection(const bool bInIsSelected);

	/** Call OnPoseWatchChanged for each pose watched node **/
	void NotifyAllNodesOnPoseWatchChanged(const bool IsPoseWatchActive);

	/** Called to notify all Nodes before any change to node selection or pose watch status **/
	void ReleaseAllManagedNodes();

	/** Called to notify all Nodes after any change to node selection or pose watch status **/
	void AcquireAllManagedNodes();

	/** Helper function used to keep skeletal controls in preview & instance in sync */
	struct FAnimNode_Base* FindAnimNode(class UAnimGraphNode_Base* AnimGraphNode) const;

	/** Handle a pin's default value changing be propagating it to the preview */
	void HandlePinDefaultValueChanged(UEdGraphPin* InPinThatChanged);

	/** Handle the preview mesh changing (so we can re-hook debug anim links etc.) */
	void HandlePreviewMeshChanged(USkeletalMesh* OldPreviewMesh, USkeletalMesh* NewPreviewMesh);

	/** Handle the viewport being created */
	void HandleViewportCreated(const TSharedRef<IPersonaViewport>& InPersonaViewport);

	/** Handle the preview anim blueprint being compiled */
	void HandlePreviewAnimBlueprintCompiled(UBlueprint* InBlueprint);

	/** Enable/disable pose watch on selected nodes */
	void HandlePoseWatchSelectedNodes();

	/** Removes all pose watches created by selection from the current view */
	void RemoveAllSelectionPoseWatches();

    /**
	 * Load editor settings from disk (docking state, window pos/size, option state, etc).
	 */
	virtual void LoadEditorSettings();

	/**
	 * Saves editor settings to disk (docking state, window pos/size, option state, etc).
	 */
	virtual void SaveEditorSettings();

	void HandleAnimationSequenceBrowserCreated(const TSharedRef<IAnimationSequenceBrowser>& InSequenceBrowser);

	/** Hook the BP exception handler to deal with infinite loops (more) gracefully */
	void HandleScriptException(const UObject* InObject, const FFrame& InFrame, const FBlueprintExceptionInfo& InInfo);

	void HandleUpdateSettings(const UAnimationBlueprintEditorSettings* AnimationBlueprintEditorSettings, EPropertyChangeType::Type ChangeType);

	/** Chooses a suitable pose watch color automatically - i.e. one that isn't already in use (if possible) */
	FColor ChoosePoseWatchColor() const;

	// Pose pin UI handlers
	void OnAddPosePin();
	bool CanAddPosePin() const;
	void OnRemovePosePin();
	bool CanRemovePosePin() const;

	// Node conversion functions
	void OnConvertToSequenceEvaluator();
	void OnConvertToSequencePlayer();
	void OnConvertToBlendSpaceEvaluator();
	void OnConvertToBlendSpacePlayer();
	void OnConvertToBlendSpaceGraph();
	void OnConvertToPoseBlender();
	void OnConvertToPoseByName();
	void OnConvertToAimOffsetLookAt();
	void OnConvertToAimOffsetSimple();
	void OnConvertToAimOffsetGraph();
	
	// Opens the associated asset of the selected nodes
	void OnOpenRelatedAsset();

	// State machine functions
	void OnCreateSelfTransition();
	FAnimNode_StateMachine* GetSelectedStateInstance(int32& OutSelectedState) const;
	bool CanSetActiveState() const;
	void OnSetActiveState();
	void OnReverseTransition();

	/** The extender to pass to the level editor to extend it's window menu */
	TSharedPtr<FExtender> MenuExtender;

	/** Toolbar extender */
	TSharedPtr<FExtender> ToolbarExtender;

	/** Preview instance inspector widget */
	TSharedPtr<class SWidget> PreviewEditor;

	/** Persona toolkit */
	TSharedPtr<class IPersonaToolkit> PersonaToolkit;

	/** Skeleton tree */
	TSharedPtr<class ISkeletonTree> SkeletonTree;

	// selected anim graph node 
	TArray< TWeakObjectPtr< class UAnimGraphNode_Base > > SelectedAnimGraphNodes;

	/** Sequence Browser **/
	TWeakPtr<class IAnimationSequenceBrowser> SequenceBrowser;

	/** Delegate handle registered for when pin default values change */
	FDelegateHandle OnPinDefaultValueChangedHandle;

	/** The last pin type we added to a graph's inputs */
	FEdGraphPinType LastGraphPinType;

    /** Configuration class used to store editor settings across sessions. */
	TObjectPtr<UAnimationBlueprintEditorOptions> EditorOptions;

	/** Cached mesh component held during compilation, used to reconnect debugger */
	TWeakObjectPtr<USkeletalMeshComponent> DebuggedMeshComponent;

	/** Used to track wither the editor option has changed */
	bool bPreviousPoseWatchSelectedNodes = false;

	/** Delegate handle registered for when settings change */
	FDelegateHandle AnimationBlueprintEditorSettingsChangedHandle;

	/** Delegate handle registered to handle infinite loop exceptions */
	FDelegateHandle ScriptExceptionHandle;
};
