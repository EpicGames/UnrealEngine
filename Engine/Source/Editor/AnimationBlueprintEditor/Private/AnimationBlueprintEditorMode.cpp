// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationBlueprintEditorMode.h"

#include "Animation/AnimBlueprint.h"
#include "Animation/DebugSkelMeshComponent.h"
#include "AnimationBlueprintEditor.h"
#include "BlueprintEditor.h"
#include "BlueprintEditorTabs.h"
#include "Delegates/Delegate.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "HAL/PlatformMath.h"
#include "IPersonaPreviewScene.h"
#include "IPersonaToolkit.h"
#include "ISkeletonEditorModule.h"
#include "ISkeletonTree.h"
#include "Modules/ModuleManager.h"
#include "PersonaDelegates.h"
#include "PersonaModule.h"
#include "PersonaUtils.h"
#include "SBlueprintEditorToolbar.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "AnimAssetFindReplace.h"

class UToolMenu;

/////////////////////////////////////////////////////
// FAnimationBlueprintEditorMode

namespace UE::Anim::BP::Editor
{
	TSharedRef<FTabManager::FStack> CreateLeftBottomStack(bool bIsTemplate)
	{
		TSharedRef<FTabManager::FStack> Stack = FTabManager::NewStack();

		if (!bIsTemplate)
		{
			Stack->AddTab(AnimationBlueprintEditorTabs::CurveNamesTab, ETabState::ClosedTab);
			Stack->AddTab(AnimationBlueprintEditorTabs::SkeletonTreeTab, ETabState::ClosedTab);
		}

		Stack->AddTab(AnimationBlueprintEditorTabs::PoseWatchTab, ETabState::OpenedTab);
		Stack->AddTab(FBlueprintEditorTabs::MyBlueprintID, ETabState::OpenedTab);

		return Stack;
	}

	TSharedRef<FTabManager::FStack> CreateRightBottomStack(bool bIsTemplate)
	{
		TSharedRef<FTabManager::FStack> Stack = FTabManager::NewStack();

		Stack->AddTab(AnimationBlueprintEditorTabs::AnimBlueprintPreviewEditorTab, ETabState::OpenedTab);
		Stack->AddTab(AnimationBlueprintEditorTabs::AssetBrowserTab, ETabState::OpenedTab);

		if (!bIsTemplate)
		{
			Stack->AddTab(AnimationBlueprintEditorTabs::SlotNamesTab, ETabState::ClosedTab);
		}

		Stack->SetForegroundTab(AnimationBlueprintEditorTabs::AnimBlueprintPreviewEditorTab);

		return Stack;
	}
}

FAnimationBlueprintEditorMode::FAnimationBlueprintEditorMode(const TSharedRef<FAnimationBlueprintEditor>& InAnimationBlueprintEditor)
	: FBlueprintEditorApplicationMode(InAnimationBlueprintEditor, FAnimationBlueprintEditorModes::AnimationBlueprintEditorMode, FAnimationBlueprintEditorModes::GetLocalizedMode, false, false)
{
	PreviewScenePtr = InAnimationBlueprintEditor->GetPreviewScene();
	AnimBlueprintPtr = CastChecked<UAnimBlueprint>(InAnimationBlueprintEditor->GetBlueprintObj());

	bool bIsTemplate = false;
	if (UAnimBlueprint* AnimBlueprint = AnimBlueprintPtr.Get())
	{
		bIsTemplate = AnimBlueprint->bIsTemplate;
	}

	TabLayout = FTabManager::NewLayout( "Stanalone_AnimationBlueprintEditMode_Layout_v1.6" )
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				// Main application area
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					// Left side
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.25f)
					->SetOrientation(Orient_Vertical)
					->Split
					(
						// Left top - viewport
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->SetHideTabWell(true)
						->AddTab(AnimationBlueprintEditorTabs::ViewportTab, ETabState::OpenedTab)
					)
					->Split
					(
						//	Left bottom - preview settings
						UE::Anim::BP::Editor::CreateLeftBottomStack(bIsTemplate)
						->SetSizeCoefficient(0.5f)
					)
				)
				->Split
				(
					// Middle 
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.55f)
					->Split
					(
						// Middle top - document edit area
						FTabManager::NewStack()
						->SetSizeCoefficient(0.8f)
						->AddTab("Document", ETabState::ClosedTab)
					)
					->Split
					(
						// Middle bottom - compiler results & find
						FTabManager::NewStack()
						->SetSizeCoefficient(0.2f)
						->AddTab(FBlueprintEditorTabs::CompilerResultsID, ETabState::ClosedTab)
						->AddTab(FBlueprintEditorTabs::FindResultsID, ETabState::ClosedTab)
						->AddTab(AnimationBlueprintEditorTabs::FindReplaceTab, ETabState::ClosedTab)
					)
				)
				->Split
				(
					// Right side
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.2f)
					->SetOrientation(Orient_Vertical)
					->Split
					(
						// Right top - selection details panel & overrides
						FTabManager::NewStack()
						->SetHideTabWell(false)
						->SetSizeCoefficient(0.5f)
						->AddTab(FBlueprintEditorTabs::DetailsID, ETabState::OpenedTab)
						->AddTab(AnimationBlueprintEditorTabs::AdvancedPreviewTab, ETabState::OpenedTab)
						->AddTab(AnimationBlueprintEditorTabs::AssetOverridesTab, ETabState::ClosedTab)
						->SetForegroundTab(FBlueprintEditorTabs::DetailsID)
					)
					->Split
					(
						// Right bottom - Asset browser & advanced preview settings
						UE::Anim::BP::Editor::CreateRightBottomStack(bIsTemplate)
						->SetHideTabWell(false)
						->SetSizeCoefficient(0.5f)
					)
				)
			)
		);

	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");

	FPersonaViewportArgs ViewportArgs(InAnimationBlueprintEditor->GetPersonaToolkit()->GetPreviewScene());
	ViewportArgs.OnViewportCreated = FOnViewportCreated::CreateSP(&InAnimationBlueprintEditor.Get(), &FAnimationBlueprintEditor::HandleViewportCreated);	ViewportArgs.BlueprintEditor = InAnimationBlueprintEditor;
	ViewportArgs.bShowStats = false;
	ViewportArgs.ContextName = TEXT("AnimationBlueprintEditor.Viewport");

	PersonaModule.RegisterPersonaViewportTabFactories(TabFactories, InAnimationBlueprintEditor, ViewportArgs);

	TabFactories.RegisterFactory(PersonaModule.CreateAdvancedPreviewSceneTabFactory(InAnimationBlueprintEditor, InAnimationBlueprintEditor->GetPersonaToolkit()->GetPreviewScene()));
	TabFactories.RegisterFactory(PersonaModule.CreateAnimationAssetBrowserTabFactory(InAnimationBlueprintEditor, InAnimationBlueprintEditor->GetPersonaToolkit(), FOnOpenNewAsset::CreateSP(&InAnimationBlueprintEditor.Get(), &FAnimationBlueprintEditor::HandleOpenNewAsset), FOnAnimationSequenceBrowserCreated(), true));
	TabFactories.RegisterFactory(PersonaModule.CreateAnimBlueprintPreviewTabFactory(InAnimationBlueprintEditor, InAnimationBlueprintEditor->GetPersonaToolkit()->GetPreviewScene()));
	TabFactories.RegisterFactory(PersonaModule.CreateAnimBlueprintAssetOverridesTabFactory(InAnimationBlueprintEditor, InAnimationBlueprintEditor->GetPersonaToolkit()->GetAnimBlueprint(), InAnimationBlueprintEditor->OnPostUndo));
	TabFactories.RegisterFactory(PersonaModule.CreatePoseWatchTabFactory(InAnimationBlueprintEditor));
	TSharedPtr<ISkeletonTree> SkeletonTree = InAnimationBlueprintEditor->GetSkeletonTree();
	TabFactories.RegisterFactory(PersonaModule.CreateCurveViewerTabFactory(InAnimationBlueprintEditor, SkeletonTree.IsValid() ? SkeletonTree->GetEditableSkeleton().ToSharedPtr() : nullptr, InAnimationBlueprintEditor->GetPersonaToolkit()->GetPreviewScene(), FOnObjectsSelected::CreateSP(&InAnimationBlueprintEditor.Get(), &FAnimationBlueprintEditor::HandleObjectsSelected)));
	TabFactories.RegisterFactory(PersonaModule.CreateAttributeViewerTabFactory(InAnimationBlueprintEditor, InAnimationBlueprintEditor->GetPersonaToolkit()->GetPreviewScene()));
	TabFactories.RegisterFactory(PersonaModule.CreateAnimAssetFindReplaceTabFactory(InAnimationBlueprintEditor, FAnimAssetFindReplaceConfig()));

	if (!bIsTemplate)
	{
		ISkeletonEditorModule& SkeletonEditorModule = FModuleManager::LoadModuleChecked<ISkeletonEditorModule>("SkeletonEditor");
		TabFactories.RegisterFactory(SkeletonEditorModule.CreateSkeletonTreeTabFactory(InAnimationBlueprintEditor, InAnimationBlueprintEditor->GetSkeletonTree().ToSharedRef()));
		TabFactories.RegisterFactory(PersonaModule.CreateSkeletonSlotNamesTabFactory(InAnimationBlueprintEditor, InAnimationBlueprintEditor->GetSkeletonTree()->GetEditableSkeleton(), FOnObjectSelected::CreateSP(&InAnimationBlueprintEditor.Get(), &FAnimationBlueprintEditor::HandleObjectSelected)));
	}

	// setup toolbar - clear existing toolbar extender from the BP mode
	//@TODO: Keep this in sync with BlueprintEditorModes.cpp
	ToolbarExtender = MakeShareable(new FExtender);

	if (UToolMenu* Toolbar = InAnimationBlueprintEditor->RegisterModeToolbarIfUnregistered(GetModeName()))
	{
		InAnimationBlueprintEditor->GetToolbarBuilder()->AddCompileToolbar(Toolbar);
		InAnimationBlueprintEditor->GetToolbarBuilder()->AddScriptingToolbar(Toolbar);
		InAnimationBlueprintEditor->GetToolbarBuilder()->AddBlueprintGlobalOptionsToolbar(Toolbar);
		InAnimationBlueprintEditor->GetToolbarBuilder()->AddDebuggingToolbar(Toolbar);
	}

	PersonaModule.OnRegisterTabs().Broadcast(TabFactories, InAnimationBlueprintEditor);
	LayoutExtender = MakeShared<FLayoutExtender>();
	PersonaModule.OnRegisterLayoutExtensions().Broadcast(*LayoutExtender.Get());
	TabLayout->ProcessExtensions(*LayoutExtender.Get());
}

void FAnimationBlueprintEditorMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	
	BP->RegisterToolbarTab(InTabManager.ToSharedRef());

	// Mode-specific setup
	BP->PushTabFactories(CoreTabFactories);
	BP->PushTabFactories(BlueprintEditorTabFactories);
	BP->PushTabFactories(TabFactories);
}

void FAnimationBlueprintEditorMode::PreDeactivateMode()
{
	if (UAnimBlueprint* AnimBlueprint = AnimBlueprintPtr.Get())
	{
		AnimBlueprint->SetObjectBeingDebugged(nullptr);
	}
	FBlueprintEditorApplicationMode::PreDeactivateMode();
}

void FAnimationBlueprintEditorMode::PostActivateMode()
{
	if (UAnimBlueprint* AnimBlueprint = AnimBlueprintPtr.Get())
	{
		// Switch off any active preview when going to graph editing mode
		PreviewScenePtr.Pin()->SetPreviewAnimationAsset(NULL, false);

		// When switching to anim blueprint mode, make sure the object being debugged is either a valid world object or the preview instance
		UDebugSkelMeshComponent* PreviewComponent = PreviewScenePtr.Pin()->GetPreviewMeshComponent();
		if ((AnimBlueprint->GetObjectBeingDebugged() == NULL) && (PreviewComponent->IsAnimBlueprintInstanced()))
		{
			PersonaUtils::SetObjectBeingDebugged(AnimBlueprint, PreviewComponent->GetAnimInstance());
		}

		// If we are a derived anim blueprint always show the overrides tab
		if(UAnimBlueprint::FindRootAnimBlueprint(AnimBlueprint))
		{
			MyBlueprintEditor.Pin()->GetTabManager()->TryInvokeTab(AnimationBlueprintEditorTabs::AssetOverridesTab);
		}
	}

	FBlueprintEditorApplicationMode::PostActivateMode();
}
