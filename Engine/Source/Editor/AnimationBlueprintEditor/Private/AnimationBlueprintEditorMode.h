// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditorModes.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

class FAnimationBlueprintEditor;
class FTabManager;

/////////////////////////////////////////////////////
// FAnimBlueprintEditAppMode

class FAnimationBlueprintEditorMode : public FBlueprintEditorApplicationMode
{
protected:
	// Set of spawnable tabs in persona mode (@TODO: Multiple lists!)
	FWorkflowAllowedTabSet TabFactories;

public:
	FAnimationBlueprintEditorMode(const TSharedRef<FAnimationBlueprintEditor>& InAnimationBlueprintEditor);

	// FApplicationMode interface
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	virtual void PreDeactivateMode() override;
	virtual void PostActivateMode() override;
	// End of FApplicationMode interface

private:
	TWeakPtr<class IPersonaPreviewScene> PreviewScenePtr;

	TWeakObjectPtr<class UAnimBlueprint> AnimBlueprintPtr;
};
