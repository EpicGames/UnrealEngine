// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationBlueprintEditorModule.h"

#include "Animation/AnimInstance.h"
#include "AnimationBlueprintEditor.h"
#include "AnimationBlueprintEditorSettings.h"
#include "AnimationGraphFactory.h"
#include "BlueprintEditor.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphSchema_K2.h"
#include "EdGraphToken.h"
#include "EdGraphUtilities.h"
#include "Engine/Blueprint.h"
#include "HAL/Platform.h"
#include "ISettingsModule.h"
#include "Internationalization/Internationalization.h"
#include "K2Node_CallFunction.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "MessageLogInitializationOptions.h"
#include "MessageLogModule.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "BlueprintActionDatabaseRegistrar.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "String/ParseTokens.h"
#include "K2Node_Event.h"
#include "AnimNotifyEventNodeSpawner.h"
#include "BlueprintActionDatabase.h"
#include "Animation/Skeleton.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "IMessageLogListing.h"
#include "Misc/UObjectToken.h"

class IAnimationBlueprintEditor;
class IToolkitHost;

IMPLEMENT_MODULE( FAnimationBlueprintEditorModule, AnimationBlueprintEditor);

#define LOCTEXT_NAMESPACE "AnimationBlueprintEditorModule"

void FAnimationBlueprintEditorModule::StartupModule()
{
	MenuExtensibilityManager = MakeShareable(new FExtensibilityManager);
	ToolBarExtensibilityManager = MakeShareable(new FExtensibilityManager);

	AnimGraphNodeFactory = MakeShareable(new FAnimationGraphNodeFactory());
	FEdGraphUtilities::RegisterVisualNodeFactory(AnimGraphNodeFactory);

	AnimGraphPinFactory = MakeShareable(new FAnimationGraphPinFactory());
	FEdGraphUtilities::RegisterVisualPinFactory(AnimGraphPinFactory);

	AnimGraphPinConnectionFactory = MakeShareable(new FAnimationGraphPinConnectionFactory());
	FEdGraphUtilities::RegisterVisualPinConnectionFactory(AnimGraphPinConnectionFactory);

	FKismetEditorUtilities::RegisterOnBlueprintCreatedCallback(this, UAnimInstance::StaticClass(), FKismetEditorUtilities::FOnBlueprintCreated::CreateRaw(this, &FAnimationBlueprintEditorModule::OnNewBlueprintCreated));
	
	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	FMessageLogInitializationOptions InitOptions;
	InitOptions.bShowFilters = true;
	InitOptions.bShowPages = true;
	MessageLogModule.RegisterLogListing("AnimBlueprintLog", LOCTEXT("AnimBlueprintLog", "Anim Blueprint Log"), InitOptions);
	
	TSharedRef<IMessageLogListing> LogListing = MessageLogModule.GetLogListing("AnimBlueprintLog");
	LogListing->OnMessageTokenClicked().AddRaw(this, &FAnimationBlueprintEditorModule::HandleMessageTokenClicked);

	ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");
	SettingsModule.RegisterSettings("Editor", "ContentEditors", "AnimationBlueprintEditor",
		LOCTEXT("AnimationBlueprintEditorSettingsName", "Animation Blueprint Editor"),
		LOCTEXT("AnimationBlueprintEditorSettingsDescription", "Configure the look and feel of the Animation Blueprint Editor."),
		GetMutableDefault<UAnimationBlueprintEditorSettings>());

}

void FAnimationBlueprintEditorModule::ShutdownModule()
{
	FKismetEditorUtilities::UnregisterAutoBlueprintNodeCreation(this);

	FEdGraphUtilities::UnregisterVisualNodeFactory(AnimGraphNodeFactory);
	FEdGraphUtilities::UnregisterVisualPinFactory(AnimGraphPinFactory);
	FEdGraphUtilities::UnregisterVisualPinConnectionFactory(AnimGraphPinConnectionFactory);

	FMessageLogModule& MessageLogModule = FModuleManager::LoadModuleChecked<FMessageLogModule>("MessageLog");
	TSharedRef<IMessageLogListing> LogListing = MessageLogModule.GetLogListing("AnimBlueprintLog");
	LogListing->OnMessageTokenClicked().RemoveAll(this);

	ISettingsModule& SettingsModule = FModuleManager::LoadModuleChecked<ISettingsModule>("Settings");
	SettingsModule.UnregisterSettings("Editor", "ContentEditors", "AnimationBlueprintEditor");

	MenuExtensibilityManager.Reset();
	ToolBarExtensibilityManager.Reset();
}

TSharedRef<IAnimationBlueprintEditor> FAnimationBlueprintEditorModule::CreateAnimationBlueprintEditor( const EToolkitMode::Type Mode, const TSharedPtr< IToolkitHost >& InitToolkitHost, class UAnimBlueprint* InAnimBlueprint)
{
	TSharedRef< FAnimationBlueprintEditor > NewAnimationBlueprintEditor( new FAnimationBlueprintEditor() );
	NewAnimationBlueprintEditor->InitAnimationBlueprintEditor( Mode, InitToolkitHost, InAnimBlueprint);
	return NewAnimationBlueprintEditor;
}

void FAnimationBlueprintEditorModule::GetTypeActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	if (!ActionRegistrar.IsOpenForRegistration(UAnimBlueprint::StaticClass()))
	{
		return;
	}

	// Grab notifies from skeletons and anim sequences
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	FARFilter Filter;
	Filter.bRecursiveClasses = true;

	if (FBlueprintActionDatabase::IsClassAllowed(UAnimSequenceBase::StaticClass(), FBlueprintActionDatabase::EPermissionsContext::Asset))
	{
		Filter.ClassPaths.Add(UAnimSequenceBase::StaticClass()->GetClassPathName());
	}

	if (FBlueprintActionDatabase::IsClassAllowed(USkeleton::StaticClass(), FBlueprintActionDatabase::EPermissionsContext::Asset))
	{
		Filter.ClassPaths.Add(USkeleton::StaticClass()->GetClassPathName());
	}

	if (Filter.ClassPaths.Num() == 0)
	{
		return;
	}

	TArray<FAssetData> FoundAssetData;
	AssetRegistryModule.Get().GetAssets(Filter, FoundAssetData);

	TMap<FSoftObjectPath, TSet<FName>> NotifiesPerSkeleton;
	for (const FAssetData& AssetData : FoundAssetData)
	{
		const FString TagValue = AssetData.GetTagValueRef<FString>(USkeleton::AnimNotifyTag);
		if (!TagValue.IsEmpty())
		{
			FSoftObjectPath SkeletonPath;
			if(AssetData.GetClass() == USkeleton::StaticClass())
			{
				SkeletonPath = AssetData.ToSoftObjectPath();
			}
			else
			{
				SkeletonPath = FSoftObjectPath(AssetData.GetTagValueRef<FString>("Skeleton"));
			}
			TSet<FName>& NotifyNames = NotifiesPerSkeleton.FindOrAdd(SkeletonPath);

			UE::String::ParseTokens(TagValue, USkeleton::AnimNotifyTagDelimiter, [&NotifyNames](FStringView InToken)
			{
				FName NotifyName(InToken);
				if(NotifyName != NAME_None)
				{
					NotifyNames.Add(NotifyName);
				}
			}, UE::String::EParseTokensOptions::SkipEmpty);
		}
	}

	for (const TPair<FSoftObjectPath, TSet<FName>>& SkeletonPathNamesPair : NotifiesPerSkeleton)
	{
		for (FName NotifyName : SkeletonPathNamesPair.Value)
		{
			UAnimNotifyEventNodeSpawner* NodeSpawner = UAnimNotifyEventNodeSpawner::Create(SkeletonPathNamesPair.Key, NotifyName);
			ActionRegistrar.AddBlueprintAction(UAnimBlueprint::StaticClass(), NodeSpawner);
		}
	}
}

void FAnimationBlueprintEditorModule::GetInstanceActions(const UAnimBlueprint* InAnimBlueprint, FBlueprintActionDatabaseRegistrar& ActionRegistrar) const
{
	if (UAnimBlueprintGeneratedClass* GeneratedClass = InAnimBlueprint->GetAnimBlueprintGeneratedClass())
	{
		FSoftObjectPath SkeletonPath;
		if (InAnimBlueprint->TargetSkeleton)
		{
			SkeletonPath = FSoftObjectPath(InAnimBlueprint->TargetSkeleton);
		}

		for (int32 NotifyIdx = 0; NotifyIdx < GeneratedClass->GetAnimNotifies().Num(); NotifyIdx++)
		{
			FName NotifyName = GeneratedClass->GetAnimNotifies()[NotifyIdx].NotifyName;
			if (NotifyName != NAME_None)
			{
				UAnimNotifyEventNodeSpawner* NodeSpawner = UAnimNotifyEventNodeSpawner::Create(SkeletonPath, NotifyName);
				ActionRegistrar.AddBlueprintAction(GeneratedClass, NodeSpawner);
			}
		}
	}
}

void FAnimationBlueprintEditorModule::OnNewBlueprintCreated(UBlueprint* InBlueprint)
{
	if (InBlueprint->UbergraphPages.Num() > 0)
	{
		UEdGraph* EventGraph = InBlueprint->UbergraphPages[0];

		int32 SafeXPosition = 0;
		int32 SafeYPosition = 0;

		if (EventGraph->Nodes.Num() != 0)
		{
			SafeXPosition = EventGraph->Nodes[0]->NodePosX;
			SafeYPosition = EventGraph->Nodes[EventGraph->Nodes.Num() - 1]->NodePosY + EventGraph->Nodes[EventGraph->Nodes.Num() - 1]->NodeHeight + 100;
		}

		// add try get owner node
		UK2Node_CallFunction* GetOwnerNode = NewObject<UK2Node_CallFunction>(EventGraph);
		UFunction* MakeNodeFunction = UAnimInstance::StaticClass()->FindFunctionByName(GET_FUNCTION_NAME_CHECKED(UAnimInstance, TryGetPawnOwner));
		GetOwnerNode->CreateNewGuid();
		GetOwnerNode->PostPlacedNewNode();
		GetOwnerNode->SetFromFunction(MakeNodeFunction);
		GetOwnerNode->SetFlags(RF_Transactional);
		GetOwnerNode->AllocateDefaultPins();
		GetOwnerNode->NodePosX = SafeXPosition;
		GetOwnerNode->NodePosY = SafeYPosition;
		UEdGraphSchema_K2::SetNodeMetaData(GetOwnerNode, FNodeMetadata::DefaultGraphNode);
		GetOwnerNode->MakeAutomaticallyPlacedGhostNode();

		EventGraph->AddNode(GetOwnerNode);
	}
}

void FAnimationBlueprintEditorModule::HandleMessageTokenClicked(const TSharedRef<IMessageToken>& InMessageToken)
{
	if (InMessageToken->GetType() == EMessageToken::Object)
	{
		const TSharedRef<FUObjectToken> ObjectToken = StaticCastSharedRef<FUObjectToken>(InMessageToken);
		UEdGraphNode* Node = Cast<UEdGraphNode>(ObjectToken->GetObject().Get());
		if (Node)
		{
			UEdGraph* Graph = Node->GetGraph();
			if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Graph->GetOuter()))
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AnimBlueprint);
				if (IAnimationBlueprintEditor* AnimBlueprintEditor =
					static_cast<IAnimationBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AnimBlueprint, true)))
				{
					AnimBlueprintEditor->JumpToHyperlink(Node);
				}
			}
		}
	}
	else if (InMessageToken->GetType() == EMessageToken::EdGraph)
	{
		const TSharedRef<FEdGraphToken> EdGraphToken = StaticCastSharedRef<FEdGraphToken>(InMessageToken);
		const UEdGraphPin* Pin = EdGraphToken->GetPin();
		const UEdGraphNode* Node = Cast<UEdGraphNode>(EdGraphToken->GetGraphObject());
		if (Pin)
		{
			UEdGraph* Graph = Pin->GetOwningNode()->GetGraph();
			if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Graph->GetOuter()))
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AnimBlueprint);
				if (IAnimationBlueprintEditor* AnimBlueprintEditor =
					static_cast<IAnimationBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AnimBlueprint, true)))
				{
					AnimBlueprintEditor->JumpToPin(Pin);
				}
			}
		}
		else if (Node)
		{
			UEdGraph* Graph = Node->GetGraph();
			if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Graph->GetOuter()))
			{
				GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset(AnimBlueprint);
				if (IAnimationBlueprintEditor* AnimBlueprintEditor =
					static_cast<IAnimationBlueprintEditor*>(GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AnimBlueprint, true)))
				{
					AnimBlueprintEditor->JumpToHyperlink(Node);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
