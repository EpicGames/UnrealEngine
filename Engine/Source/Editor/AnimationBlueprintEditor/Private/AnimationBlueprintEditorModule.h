// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "IAnimationBlueprintEditorModule.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/IToolkit.h"

class FExtensibilityManager;
struct FAnimationGraphNodeFactory;
struct FAnimationGraphPinConnectionFactory;
struct FAnimationGraphPinFactory;

/**
 * Animation Blueprint Editor module allows editing of Animation Blueprints
 */
class FAnimationBlueprintEditorModule : public IAnimationBlueprintEditorModule
{
public:
	/**
	 * Called right after the module's DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule();

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule();

	/** IAnimationBlueprintEditorModule interface */
	virtual TSharedRef<class IAnimationBlueprintEditor> CreateAnimationBlueprintEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UAnimBlueprint* Blueprint) override;
	virtual TArray<FAnimationBlueprintEditorToolbarExtender>& GetAllAnimationBlueprintEditorToolbarExtenders() { return AnimationBlueprintEditorToolbarExtenders; }
	virtual void GetTypeActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;
	virtual void GetInstanceActions(const UAnimBlueprint* InAnimBlueprint, FBlueprintActionDatabaseRegistrar& ActionRegistrar) const override;

	/** Gets the extensibility managers for outside entities to extend this editor's menus and toolbars */
	virtual TSharedPtr<FExtensibilityManager> GetMenuExtensibilityManager() override { return MenuExtensibilityManager; }
	virtual TSharedPtr<FExtensibilityManager> GetToolBarExtensibilityManager() override { return ToolBarExtensibilityManager; }

	virtual FOnGetCustomDebugObjects& OnGetCustomDebugObjects() override { return OnGetCustomDebugObjectsDelegate; }

private:
	/** When a new AnimBlueprint is created, this will handle post creation work such as adding non-event default nodes */
	void OnNewBlueprintCreated(class UBlueprint* InBlueprint);
	void HandleMessageTokenClicked(const TSharedRef<IMessageToken>& InMessageToken);

private:
	TSharedPtr<class FExtensibilityManager> MenuExtensibilityManager;
	TSharedPtr<class FExtensibilityManager> ToolBarExtensibilityManager;

	TArray<FAnimationBlueprintEditorToolbarExtender> AnimationBlueprintEditorToolbarExtenders;

	TSharedPtr<FAnimationGraphNodeFactory> AnimGraphNodeFactory;
	TSharedPtr<FAnimationGraphPinFactory> AnimGraphPinFactory;
	TSharedPtr<FAnimationGraphPinConnectionFactory> AnimGraphPinConnectionFactory;

	FOnGetCustomDebugObjects OnGetCustomDebugObjectsDelegate;
};
