// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationBlueprintEditorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimationBlueprintEditorSettings)

void UAnimationBlueprintEditorSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	OnSettingsChange.Broadcast(this, PropertyChangedEvent.ChangeType);
}
