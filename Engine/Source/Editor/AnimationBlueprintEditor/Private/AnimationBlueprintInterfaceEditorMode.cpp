// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationBlueprintInterfaceEditorMode.h"

#include "Animation/AnimBlueprint.h"
#include "AnimationBlueprintEditor.h"
#include "BlueprintEditor.h"
#include "BlueprintEditorTabs.h"
#include "Framework/Docking/LayoutExtender.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Modules/ModuleManager.h"
#include "PersonaModule.h"
#include "SBlueprintEditorToolbar.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/WeakObjectPtr.h"

class UToolMenu;

FAnimationBlueprintInterfaceEditorMode::FAnimationBlueprintInterfaceEditorMode(const TSharedRef<FAnimationBlueprintEditor>& InAnimationBlueprintEditor)
	: FBlueprintInterfaceApplicationMode(InAnimationBlueprintEditor, FAnimationBlueprintEditorModes::AnimationBlueprintInterfaceEditorMode, FAnimationBlueprintEditorModes::GetLocalizedMode)
{
	AnimBlueprintPtr = CastChecked<UAnimBlueprint>(InAnimationBlueprintEditor->GetBlueprintObj());

	TabLayout = FTabManager::NewLayout( "Standalone_AnimationBlueprintInterfaceEditMode_Layout_v1.1" )
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Vertical)
			->Split
			(
				// Main application area
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Horizontal)
				->Split
				(
					// Middle 
					FTabManager::NewSplitter()
					->SetOrientation(Orient_Vertical)
					->SetSizeCoefficient(0.75f)
					->Split
					(
						// Middle top - document edit area
						FTabManager::NewStack()
						->SetSizeCoefficient(0.8f)
						->AddTab("Document", ETabState::ClosedTab)
					)
					->Split
					(
						// Middle bottom - compiler results & find
						FTabManager::NewStack()
						->SetSizeCoefficient(0.2f)
						->AddTab(FBlueprintEditorTabs::CompilerResultsID, ETabState::ClosedTab)
						->AddTab(FBlueprintEditorTabs::FindResultsID, ETabState::ClosedTab)
					)
				)
				->Split
				(
					// Right side
					FTabManager::NewSplitter()
					->SetSizeCoefficient(0.25f)
					->SetOrientation(Orient_Vertical)
					->Split
					(
						// Right top - details and my blueprint
						FTabManager::NewStack()
						->SetHideTabWell(false)
						->SetSizeCoefficient(0.5f)
						->AddTab(FBlueprintEditorTabs::MyBlueprintID, ETabState::OpenedTab)
					)
					->Split
					(
						// Middle bottom - compiler results & find
						FTabManager::NewStack()
						->SetSizeCoefficient(0.5f)
						->AddTab(FBlueprintEditorTabs::DetailsID, ETabState::OpenedTab)
					)
				)
			)
		);

	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");

	ToolbarExtender = MakeShareable(new FExtender);

	if (UToolMenu* Toolbar = InAnimationBlueprintEditor->RegisterModeToolbarIfUnregistered(GetModeName()))
	{
		InAnimationBlueprintEditor->GetToolbarBuilder()->AddCompileToolbar(Toolbar);
		InAnimationBlueprintEditor->GetToolbarBuilder()->AddBlueprintGlobalOptionsToolbar(Toolbar);	
	}

	PersonaModule.OnRegisterTabs().Broadcast(TabFactories, InAnimationBlueprintEditor);
	LayoutExtender = MakeShared<FLayoutExtender>();
	PersonaModule.OnRegisterLayoutExtensions().Broadcast(*LayoutExtender.Get());
	TabLayout->ProcessExtensions(*LayoutExtender.Get());
}

void FAnimationBlueprintInterfaceEditorMode::RegisterTabFactories(TSharedPtr<FTabManager> InTabManager)
{
	TSharedPtr<FBlueprintEditor> BP = MyBlueprintEditor.Pin();
	
	BP->RegisterToolbarTab(InTabManager.ToSharedRef());

	// Mode-specific setup
	BP->PushTabFactories(BlueprintInterfaceTabFactories);
}
