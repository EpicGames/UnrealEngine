// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "BlueprintEditorModes.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

class FAnimationBlueprintEditor;
class FTabManager;

class FAnimationBlueprintInterfaceEditorMode : public FBlueprintInterfaceApplicationMode
{
protected:
	// Set of spawnable tabs in persona mode (@TODO: Multiple lists!)
	FWorkflowAllowedTabSet TabFactories;

public:
	FAnimationBlueprintInterfaceEditorMode(const TSharedRef<FAnimationBlueprintEditor>& InAnimationBlueprintEditor);

	// FApplicationMode interface
	virtual void RegisterTabFactories(TSharedPtr<FTabManager> InTabManager) override;
	// End of FApplicationMode interface

private:
	TWeakObjectPtr<class UAnimBlueprint> AnimBlueprintPtr;
};
