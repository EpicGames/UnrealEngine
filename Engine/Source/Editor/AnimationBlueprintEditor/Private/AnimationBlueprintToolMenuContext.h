// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AnimationBlueprintToolMenuContext.generated.h"

class FAnimationBlueprintEditor;

UCLASS()
class UAnimationBlueprintToolMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<FAnimationBlueprintEditor> AnimationBlueprintEditor;
};