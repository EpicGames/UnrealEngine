// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationGraphFactory.h"

#include "AnimGraphConnectionDrawingPolicy.h"
#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_BlendSpaceBase.h"
#include "AnimGraphNode_BlendSpaceGraphBase.h"
#include "AnimGraphNode_LayeredBoneBlend.h"
#include "AnimGraphNode_LinkedAnimGraph.h"
#include "AnimGraphNode_Root.h"
#include "AnimGraphNode_SequencePlayer.h"
#include "AnimGraphNode_StateMachineBase.h"
#include "AnimStateAliasNode.h"
#include "AnimStateConduitNode.h"
#include "AnimStateEntryNode.h"
#include "AnimStateNode.h"
#include "AnimStateTransitionNode.h"
#include "Animation/AnimNodeBase.h"
#include "AnimationGraphSchema.h"
#include "AnimationNodes/SAnimNodeReference.h"
#include "AnimationNodes/SAnimationGraphNode.h"
#include "AnimationNodes/SGraphNodeAnimationResult.h"
#include "AnimationNodes/SGraphNodeBlendSpaceGraph.h"
#include "AnimationNodes/SGraphNodeBlendSpacePlayer.h"
#include "AnimationNodes/SGraphNodeLayeredBoneBlend.h"
#include "AnimationNodes/SGraphNodeLinkedLayer.h"
#include "AnimationNodes/SGraphNodeSequencePlayer.h"
#include "AnimationNodes/SGraphNodeStateMachineInstance.h"
#include "AnimationPins/SGraphPinPose.h"
#include "AnimationStateMachineSchema.h"
#include "AnimationStateNodes/SGraphNodeAnimState.h"
#include "AnimationStateNodes/SGraphNodeAnimStateAlias.h"
#include "AnimationStateNodes/SGraphNodeAnimStateEntry.h"
#include "AnimationStateNodes/SGraphNodeAnimTransition.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "EdGraph/EdGraphSchema.h"
#include "EdGraphSchema_K2.h"
#include "K2Node_AnimNodeReference.h"
#include "KismetPins/SGraphPinExec.h"
#include "StateMachineConnectionDrawingPolicy.h"
#include "Templates/Casts.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

TSharedPtr<class SGraphNode> FAnimationGraphNodeFactory::CreateNode(class UEdGraphNode* InNode) const 
{
	if (UAnimGraphNode_Base* BaseAnimNode = Cast<UAnimGraphNode_Base>(InNode))
	{
		if (UAnimGraphNode_Root* RootAnimNode = Cast<UAnimGraphNode_Root>(InNode))
		{
			return SNew(SGraphNodeAnimationResult, RootAnimNode);
		}
		else if (UAnimGraphNode_StateMachineBase* StateMachineInstance = Cast<UAnimGraphNode_StateMachineBase>(InNode))
		{
			return SNew(SGraphNodeStateMachineInstance, StateMachineInstance);
		}
		else if (UAnimGraphNode_SequencePlayer* SequencePlayer = Cast<UAnimGraphNode_SequencePlayer>(InNode))
		{
			return SNew(SGraphNodeSequencePlayer, SequencePlayer);
		}
		else if (UAnimGraphNode_LayeredBoneBlend* LayeredBlend = Cast<UAnimGraphNode_LayeredBoneBlend>(InNode))
		{
			return SNew(SGraphNodeLayeredBoneBlend, LayeredBlend);
		}
		else if (UAnimGraphNode_BlendSpaceBase* BlendSpacePlayer = Cast<UAnimGraphNode_BlendSpaceBase>(InNode))
		{
			return SNew(SGraphNodeBlendSpacePlayer, BlendSpacePlayer);
		}
		else if (UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceGraph = Cast<UAnimGraphNode_BlendSpaceGraphBase>(InNode))
		{
			return SNew(SGraphNodeBlendSpaceGraph, BlendSpaceGraph);
		}
		else if (UAnimGraphNode_LinkedAnimGraph* LinkedAnimLayer = Cast<UAnimGraphNode_LinkedAnimGraph>(InNode))
		{
			return SNew(SGraphNodeLinkedLayer, LinkedAnimLayer);
		}
		else
		{
			return SNew(SAnimationGraphNode, BaseAnimNode);
		}
	}
	else if (UAnimStateTransitionNode* TransitionNode = Cast<UAnimStateTransitionNode>(InNode))
	{
		return SNew(SGraphNodeAnimTransition, TransitionNode);
	}
	else if (UAnimStateNode* StateNode = Cast<UAnimStateNode>(InNode))
	{
		return SNew(SGraphNodeAnimState, StateNode);
	}
	else if (UAnimStateAliasNode* StateAliasNode = Cast<UAnimStateAliasNode>(InNode))
	{
		return SNew(SGraphNodeAnimStateAlias, StateAliasNode);
	}
	else if (UAnimStateConduitNode* ConduitNode = Cast<UAnimStateConduitNode>(InNode))
	{
		return SNew(SGraphNodeAnimConduit, ConduitNode);
	}
	else if (UAnimStateEntryNode* EntryNode = Cast<UAnimStateEntryNode>(InNode))
	{
		return SNew(SGraphNodeAnimStateEntry, EntryNode);
	}
	else if (UK2Node_AnimNodeReference* AnimNodeReference = Cast<UK2Node_AnimNodeReference>(InNode))
	{
		return SNew(SAnimNodeReference, AnimNodeReference);
	}
	
	return nullptr;
}

TSharedPtr<class SGraphPin> FAnimationGraphPinFactory::CreatePin(class UEdGraphPin* InPin) const
{
	if (InPin->GetSchema()->IsA<UAnimationGraphSchema>() && InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct)
	{
		if ((InPin->PinType.PinSubCategoryObject == FPoseLink::StaticStruct()) || (InPin->PinType.PinSubCategoryObject == FComponentSpacePoseLink::StaticStruct()))
		{
			return SNew(SGraphPinPose, InPin);
		}
	}

	if (InPin->GetSchema()->IsA<UAnimationStateMachineSchema>() && InPin->PinType.PinCategory == UAnimationStateMachineSchema::PC_Exec)
	{
		return SNew(SGraphPinExec, InPin);
	}

	return nullptr;
}

class FConnectionDrawingPolicy* FAnimationGraphPinConnectionFactory::CreateConnectionPolicy(const class UEdGraphSchema* Schema, int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const class FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const
{
	if (Schema->IsA(UAnimationGraphSchema::StaticClass()))
	{
		return new FAnimGraphConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements, InGraphObj);
	}
	else if (Schema->IsA(UAnimationStateMachineSchema::StaticClass()))
	{
		return new FStateMachineConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements, InGraphObj);
	}

	return nullptr;
}