// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraphUtilities.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

#define UE_API ANIMATIONBLUEPRINTEDITOR_API

struct FAnimationGraphNodeFactory : public FGraphPanelNodeFactory
{
	UE_API virtual TSharedPtr<class SGraphNode> CreateNode(class UEdGraphNode* InNode) const override;
};

struct FAnimationGraphPinFactory : public FGraphPanelPinFactory
{
public:
	UE_API virtual TSharedPtr<class SGraphPin> CreatePin(class UEdGraphPin* Pin) const override;
};

struct FAnimationGraphPinConnectionFactory : public FGraphPanelPinConnectionFactory
{
public:
	UE_API virtual class FConnectionDrawingPolicy* CreateConnectionPolicy(const class UEdGraphSchema* Schema, int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const class FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const override;
};

#undef UE_API
