// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAnimNodeReference.h"

#include "Delegates/Delegate.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "K2Node_AnimNodeReference.h"
#include "Layout/Margin.h"
#include "SGraphPin.h"
#include "SLevelOfDetailBranchNode.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
class UEdGraphPin;

#define LOCTEXT_NAMESPACE "SAnimNodeReferenceNode"

void SAnimNodeReference::Construct(const FArguments& InArgs, UK2Node_AnimNodeReference* InNode)
{
	GraphNode = InNode;
	UpdateGraphNode();
}

TSharedRef<SWidget> SAnimNodeReference::UpdateTitleWidget(FText InTitleText, TSharedPtr<SWidget> InTitleWidget, EHorizontalAlignment& InOutTitleHAlign, FMargin& InOutTitleMargin) const
{
	UK2Node_AnimNodeReference* K2Node_AnimNodeReference = CastChecked<UK2Node_AnimNodeReference>(GraphNode);

	InTitleWidget =
		SNew(SLevelOfDetailBranchNode)
		.UseLowDetailSlot(this, &SAnimNodeReference::UseLowDetailNodeTitles)
		.LowDetail()
		[
			SNew(SSpacer)
		]
		.HighDetail()
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text_Lambda([K2Node_AnimNodeReference]()
				{
					return K2Node_AnimNodeReference->GetLabelText();
				})
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("NodeSubTitle", "Anim Node Reference"))
				.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("AnimGraph.AnimNodeReference.Subtitle"))
			]
		];

	InOutTitleHAlign = HAlign_Left;
	InOutTitleMargin = FMargin(12.0f, 8.0f, 36.0f, 6.0f);
	
	return InTitleWidget.ToSharedRef();
}

TSharedPtr<SGraphPin> SAnimNodeReference::CreatePinWidget(UEdGraphPin* Pin) const
{
	TSharedPtr<SGraphPin> DefaultWidget = SGraphNodeK2Var::CreatePinWidget(Pin);
	DefaultWidget->SetShowLabel(false);

	return DefaultWidget;
}

#undef LOCTEXT_NAMESPACE