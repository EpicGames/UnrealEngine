// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "KismetNodes/SGraphNodeK2Var.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class FText;
class SGraphPin;
class SWidget;
class UEdGraphPin;
class UK2Node_AnimNodeReference;
struct FMargin;

class SAnimNodeReference : public SGraphNodeK2Var
{
public:
	SLATE_BEGIN_ARGS(SAnimNodeReference) {}
	SLATE_END_ARGS()

public:
	void Construct(const FArguments& InArgs, UK2Node_AnimNodeReference* InNode);

	// SGraphNode interface
	virtual TSharedPtr<SGraphPin> CreatePinWidget(UEdGraphPin* Pin) const override;
	
	// SGraphNodeK2Var interface
	virtual TSharedRef<SWidget> UpdateTitleWidget(FText InTitleText, TSharedPtr<SWidget> InTitleWidget, EHorizontalAlignment& InOutTitleHAlign, FMargin& InOutTitleMargin) const override;
};