// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationNodes/SAnimationGraphNode.h"

#include "AnimGraphNode_Base.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimInstance.h"
#include "AnimationGraphSchema.h"
#include "BlueprintMemberReferenceCustomization.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/Blueprint.h"
#include "Engine/MemberReference.h"
#include "GenericPlatform/ICursor.h"
#include "IDetailTreeNode.h"
#include "IDocumentation.h"
#include "IPropertyRowGenerator.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/Children.h"
#include "Layout/Geometry.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorDelegates.h"
#include "PropertyEditorModule.h"
#include "PropertyHandle.h"
#include "SGraphNode.h"
#include "SGraphPanel.h"
#include "SGraphPin.h"
#include "SLevelOfDetailBranchNode.h"
#include "SNodePanel.h"
#include "SPoseWatchOverlay.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateBrush.h"
#include "Styling/SlateTypes.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SToolTip.h"
#include "Widgets/SWidget.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"

#define LOCTEXT_NAMESPACE "AnimationGraphNode"

void SAnimationGraphNode::Construct(const FArguments& InArgs, UAnimGraphNode_Base* InNode)
{
	this->GraphNode = InNode;

	this->SetCursor(EMouseCursor::CardinalCross);

	this->UpdateGraphNode();

	ReconfigurePinWidgetsForPropertyBindings(CastChecked<UAnimGraphNode_Base>(GraphNode), SharedThis(this), [this](UEdGraphPin* InPin){ return FindWidgetForPin(InPin); });

	const FSlateBrush* ImageBrush = FAppStyle::Get().GetBrush(TEXT("Graph.AnimationFastPathIndicator"));

	IndicatorWidget =
		SNew(SImage)
		.Image(ImageBrush)
		.ToolTip(IDocumentation::Get()->CreateToolTip(LOCTEXT("AnimGraphNodeIndicatorTooltip", "Fast path enabled: This node is not using any Blueprint calls to update its data."), NULL, TEXT("Shared/GraphNodes/Animation"), TEXT("GraphNode_FastPathInfo")))
		.Visibility(EVisibility::Visible);


	PoseViewWidget = SNew(SPoseWatchOverlay, InNode);

	LastHighDetailSize = FVector2D::ZeroVector;
}

void SAnimationGraphNode::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SGraphNodeK2Base::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
	
	if (CachedContentArea != nullptr && !UseLowDetailNodeContent())
	{
		LastHighDetailSize = CachedContentArea->GetTickSpaceGeometry().Size;
	}
}

TArray<FOverlayWidgetInfo> SAnimationGraphNode::GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const
{
	TArray<FOverlayWidgetInfo> Widgets;

	if (UAnimGraphNode_Base* AnimNode = CastChecked<UAnimGraphNode_Base>(GraphNode, ECastCheckedType::NullAllowed))
	{
		if (AnimNode->BlueprintUsage == EBlueprintUsage::DoesNotUseBlueprint)
		{
			const FSlateBrush* ImageBrush = FAppStyle::Get().GetBrush(TEXT("Graph.AnimationFastPathIndicator"));

			FOverlayWidgetInfo Info;
			Info.OverlayOffset = FVector2f(WidgetSize.X - (ImageBrush->ImageSize.X * 0.5f), -(ImageBrush->ImageSize.Y * 0.5f));
			Info.Widget = IndicatorWidget;

			Widgets.Add(Info);
		}

		if (PoseViewWidget->IsPoseWatchValid())
		{
			FOverlayWidgetInfo Info;
			Info.OverlayOffset = PoseViewWidget->GetOverlayOffset();
			Info.Widget = PoseViewWidget;
			Widgets.Add(Info);
		}
	}

	return Widgets;
}


TSharedRef<SWidget> SAnimationGraphNode::CreateTitleWidget(TSharedPtr<SNodeTitle> InNodeTitle)
{
	// Store title widget reference
	NodeTitle = InNodeTitle;

	// hook up invalidation delegate
	UAnimGraphNode_Base* AnimGraphNode = CastChecked<UAnimGraphNode_Base>(GraphNode);
	AnimGraphNode->OnNodeTitleChangedEvent().AddSP(this, &SAnimationGraphNode::HandleNodeTitleChanged);

	return SGraphNodeK2Base::CreateTitleWidget(InNodeTitle);
}

void SAnimationGraphNode::HandleNodeTitleChanged()
{
	if(NodeTitle.IsValid())
	{
		NodeTitle->MarkDirty();
	}
}

void SAnimationGraphNode::GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const
{
	SGraphNodeK2Base::GetNodeInfoPopups(Context, Popups);

	UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(GraphNode));
	if(AnimBlueprint)
	{
		UAnimInstance* ActiveObject = Cast<UAnimInstance>(AnimBlueprint->GetObjectBeingDebugged());
		UAnimBlueprintGeneratedClass* Class = AnimBlueprint->GetAnimBlueprintGeneratedClass();

		const FLinearColor Color(1.f, 0.5f, 0.25f);

		// Display various types of debug data
		if ((ActiveObject != NULL) && (Class != NULL))
		{
			if (Class->GetAnimNodeProperties().Num())
			{
				if(int32* NodeIndexPtr = Class->GetAnimBlueprintDebugData().NodePropertyToIndexMap.Find(TWeakObjectPtr<UAnimGraphNode_Base>(Cast<UAnimGraphNode_Base>(GraphNode))))
				{
					int32 AnimNodeIndex = *NodeIndexPtr;
					// reverse node index temporarily because of a bug in NodeGuidToIndexMap
					AnimNodeIndex = Class->GetAnimNodeProperties().Num() - AnimNodeIndex - 1;

					FString PopupText;
					for(auto & NodeValue : Class->GetAnimBlueprintDebugData().NodeValuesThisFrame)
					{
						if (NodeValue.NodeID == AnimNodeIndex)
						{
							if (PopupText.IsEmpty())
							{
								PopupText = NodeValue.Text;
							}
							else
							{
								PopupText = FString::Format(TEXT("{0}\n{1}"), {PopupText, NodeValue.Text});
							}
						}
					}
					if (!PopupText.IsEmpty())
					{
						Popups.Emplace(nullptr, Color, PopupText);
					}
				}
			}
		}
	}
}

void SAnimationGraphNode::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
{
	if (UAnimGraphNode_Base* AnimNode = CastChecked<UAnimGraphNode_Base>(GraphNode, ECastCheckedType::NullAllowed))
	{
		auto UseLowDetailNode = [this]()
		{
			return GetCurrentLOD() <= EGraphRenderingLOD::LowDetail;
		};

		// Insert above the error reporting bar
		MainBox->InsertSlot(FMath::Max(0, MainBox->NumSlots() - TagAndFunctionsSlotReverseIndex))
		.AutoHeight()
		.Padding(4.0f, 2.0f, 4.0f, 2.0f)
		[
			SNew(SVerticalBox)
			.IsEnabled_Lambda([this](){ return IsNodeEditable(); })
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				CreateNodeFunctionsWidget(AnimNode, MakeAttributeLambda(UseLowDetailNode))
			]
		];

		MainBox->InsertSlot(FMath::Max(0, MainBox->NumSlots() - TagAndFunctionsSlotReverseIndex))
		.AutoHeight()
		.Padding(4.0f, 2.0f, 4.0f, 2.0f)
		[
			SNew(SVerticalBox)
			.IsEnabled_Lambda([this](){ return IsNodeEditable(); })
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			[
				CreateNodeTagWidget(AnimNode, MakeAttributeLambda(UseLowDetailNode))
			]
		];
	}
}

TSharedRef<SWidget> SAnimationGraphNode::CreateNodeContentArea()
{
	CachedContentArea = SGraphNodeK2Base::CreateNodeContentArea();

	return SNew(SLevelOfDetailBranchNode)
		.UseLowDetailSlot(this, &SAnimationGraphNode::UseLowDetailNodeContent)
		.LowDetail()
		[
			SNew(SSpacer)
			.Size(this, &SAnimationGraphNode::GetLowDetailDesiredSize)
		]
		.HighDetail()
		[
			CachedContentArea.ToSharedRef()
		];
}

bool SAnimationGraphNode::UseLowDetailNodeContent() const
{
	if (LastHighDetailSize.IsNearlyZero())
	{
		return false;
	}

	if (const SGraphPanel* MyOwnerPanel = GetOwnerPanel().Get())
	{
		return (MyOwnerPanel->GetCurrentLOD() <= EGraphRenderingLOD::LowestDetail);
	}
	return false;
}

FVector2D SAnimationGraphNode::GetLowDetailDesiredSize() const
{
	return LastHighDetailSize;
}

// Widget used to allow functions to be viewed and edited on nodes
class SAnimNodeFunctionsWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAnimNodeFunctionsWidget) {}

	SLATE_ATTRIBUTE(bool, UseLowDetail)
	
	SLATE_END_ARGS()

	void FindFunctionBindingPropertyNode(UAnimGraphNode_Base * InNode, TSharedPtr<IPropertyHandle>& OutPropertyHandle, TSharedPtr<IDetailTreeNode>& OutDetailTreeNode, const TSharedRef<IDetailTreeNode>& InRootNode, FName InCategory, FName InMemberName)
	{
		// We already found the property we are looking for
		if (OutPropertyHandle.IsValid() || OutDetailTreeNode.IsValid())
		{
			return;
		}
		
		TArray<TSharedRef<IDetailTreeNode>> Children;
		InRootNode->GetChildren(Children);

		// Parse children
		for (const TSharedRef<IDetailTreeNode>& ChildNode : Children)
		{
			const TSharedPtr<IPropertyHandle> ChildPropertyHandle = ChildNode->CreatePropertyHandle();
				
			// Try to match node
			if (ChildPropertyHandle.IsValid() && ChildPropertyHandle->GetProperty() && ChildPropertyHandle->GetProperty()->GetFName() == InMemberName && ChildPropertyHandle->GetMetaData(TEXT("Category")) == InCategory)
			{
				OutDetailTreeNode = ChildNode;
				OutPropertyHandle = ChildPropertyHandle;
				
				// We found our function binding property.
				return;
			}

			// Look into sub categories
			FindFunctionBindingPropertyNode(InNode, OutPropertyHandle, OutDetailTreeNode, ChildNode, InCategory, InMemberName);
		}
	}
	
	void Construct(const FArguments& InArgs, UAnimGraphNode_Base* InNode)
	{
		UseLowDetail = InArgs._UseLowDetail;
		
		FPropertyEditorModule& PropertyEditorModule = FModuleManager::Get().LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyRowGenerator = PropertyEditorModule.CreatePropertyRowGenerator(FPropertyRowGeneratorArgs());
		PropertyRowGenerator->RegisterInstancedCustomPropertyTypeLayout(FMemberReference::StaticStruct()->GetFName(), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FBlueprintMemberReferenceDetails::MakeInstance));
		PropertyRowGenerator->SetObjects({ InNode });

		TSharedPtr<SGridPanel> GridPanel;

		ChildSlot
		[
			SAssignNew(GridPanel, SGridPanel)
		];

		GridPanel->SetVisibility(EVisibility::Collapsed);

		int32 RowIndex = 0;

		// Get anim node functions that are bound
		TArray<TPair<FName, FName>> FunctionBindingsInfo;
		InNode->GetBoundFunctionsInfo(FunctionBindingsInfo);

		// Build anim node widgets
		for (const TPair<FName, FName> & FunctionBindingInfo : FunctionBindingsInfo)
		{
			FName Category = FunctionBindingInfo.Key;
			FName MemberName = FunctionBindingInfo.Value;

			TSharedPtr<IPropertyHandle> PropertyHandle;
			TSharedPtr<IDetailTreeNode> DetailTreeNode;
			
			GridPanel->SetVisibility(EVisibility::Visible);

			// Find row for function binding property
			for (const TSharedRef<IDetailTreeNode>& RootTreeNode : PropertyRowGenerator->GetRootTreeNodes())
			{
				if (Category.ToString().Contains(RootTreeNode->GetNodeName().ToString()))
				{
					FindFunctionBindingPropertyNode(InNode, PropertyHandle, DetailTreeNode, RootTreeNode, Category, MemberName);
				}
			}
			
			// Build function binding widget
			if (DetailTreeNode.IsValid() && PropertyHandle.IsValid())
			{
				// Ensure anim node is rebuilt if any function binding changes
				PropertyHandle->SetOnPropertyValueChanged(FSimpleDelegate::CreateLambda([InNode]()
				{
					InNode->ReconstructNode();
				}));
				
				DetailNodes.Add(DetailTreeNode);

				const FNodeWidgets NodeWidgets = DetailTreeNode->CreateNodeWidgets();

				// Binding variable name
				GridPanel->AddSlot(0, RowIndex)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(10.0f, 2.0f, 2.0f, 2.0f)
				[
					SNew(SLevelOfDetailBranchNode)
					.UseLowDetailSlot(UseLowDetail)
					.LowDetail()
					[
						SNew(SSpacer)
						.Size(FVector2D(24.0f, 24.f))
					]
					.HighDetail()
					[
						NodeWidgets.NameWidget.ToSharedRef()
					]
				];

				// Function name
				GridPanel->AddSlot(1, RowIndex)
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(2.0f, 2.0f, 10.0f, 2.0f)
				[
					SNew(SLevelOfDetailBranchNode)
					.UseLowDetailSlot(UseLowDetail)
					.LowDetail()
					[
						SNew(SSpacer)
						.Size(FVector2D(24.0f, 24.f))
					]
					.HighDetail()
					[	
						NodeWidgets.ValueWidget.ToSharedRef()
					]
				];

				RowIndex++;
			}
		}
		
		if (DetailNodes.Num() == 0)
		{
			// If we didnt add a function binding, remove the row generator as we dont need it and its expensive (as it ticks)
			PropertyRowGenerator.Reset();
		}
	}

	// Property row generator used to display function properties on nodes
	TSharedPtr<IPropertyRowGenerator> PropertyRowGenerator;

	// Hold a reference to the root ptr of the details tree we use to display function properties
	TArray<TSharedPtr<IDetailTreeNode>> DetailNodes;

	// Attribute allowing LOD
	TAttribute<bool> UseLowDetail;
};

TSharedRef<SWidget> SAnimationGraphNode::CreateNodeFunctionsWidget(UAnimGraphNode_Base* InAnimNode, TAttribute<bool> InUseLowDetail)
{
	return SNew(SAnimNodeFunctionsWidget, InAnimNode)
		.UseLowDetail(InUseLowDetail);
}

TSharedRef<SWidget> SAnimationGraphNode::CreateNodeTagWidget(UAnimGraphNode_Base* InAnimNode, TAttribute<bool> InUseLowDetail)
{
	return SNew(SLevelOfDetailBranchNode)
		.Visibility_Lambda([InAnimNode](){ return (InAnimNode->Tag != NAME_None) ? EVisibility::Visible : EVisibility::Collapsed; })
		.UseLowDetailSlot(InUseLowDetail)
		.LowDetail()
		[
			SNew(SSpacer)
			.Size(FVector2D(24.0f, 24.f))
		]
		.HighDetail()
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 4.0f, 4.0f))
			[
				SNew(SInlineEditableTextBlock)
				.ToolTipText_Lambda([InAnimNode](){ return FText::Format(LOCTEXT("TagFormat_Tooltip", "Tag: {0}\nThis node can be referenced elsewhere in this Anim Blueprint using this tag"), FText::FromName(InAnimNode->GetTag())); })
				.Style(&FAppStyle::Get().GetWidgetStyle<FInlineEditableTextBlockStyle>("AnimGraph.Node.Tag"))
				.Text_Lambda([InAnimNode](){ return FText::FromName(InAnimNode->GetTag()); })
				.OnTextCommitted_Lambda([InAnimNode](const FText& InText, ETextCommit::Type InCommitType){ InAnimNode->SetTag(*InText.ToString()); })
			]
		];
}

void SAnimationGraphNode::ReconfigurePinWidgetsForPropertyBindings(UAnimGraphNode_Base* InAnimGraphNode, TSharedRef<SGraphNode> InGraphNodeWidget, TFunctionRef<TSharedPtr<SGraphPin>(UEdGraphPin*)> InFindWidgetForPin)
{
	for(UEdGraphPin* Pin : InAnimGraphNode->Pins)
	{
		FEdGraphPinType PinType = Pin->PinType;
		if(Pin->Direction == EGPD_Input && !UAnimationGraphSchema::IsPosePin(PinType))
		{
			TSharedPtr<SGraphPin> PinWidget = InFindWidgetForPin(Pin);

			if(PinWidget.IsValid())
			{
				// Tweak padding a little to improve extended appearance
				PinWidget->GetLabelAndValue()->SetInnerSlotPadding(FVector2D(2.0f, 0.0f));

				TAttribute<bool> bIsEnabled = MakeAttributeLambda([WeakWidget = TWeakPtr<SGraphNode>(InGraphNodeWidget)]()
				{
					return WeakWidget.IsValid() ? WeakWidget.Pin()->IsNodeEditable() : false;
				});
				TSharedPtr<SWidget> PropertyBindingWidget = UAnimationGraphSchema::MakeBindingWidgetForPin({ InAnimGraphNode }, Pin->GetFName(), true, bIsEnabled);
				if(PropertyBindingWidget.IsValid())
				{
					// Add binding widget
					PinWidget->GetLabelAndValue()->AddSlot()
					[
						PropertyBindingWidget.ToSharedRef()
					];

					// Hide any value widgets when we have bindings
					if(PinWidget->GetValueWidget() != SNullWidget::NullWidget)
					{
						PinWidget->GetValueWidget()->SetVisibility(MakeAttributeLambda([WeakPropertyBindingWidget = TWeakPtr<SWidget>(PropertyBindingWidget), WeakPinWidget = TWeakPtr<SGraphPin>(PinWidget)]()
						{
							EVisibility Visibility = EVisibility::Collapsed;

							if (WeakPinWidget.IsValid())
							{
								Visibility = WeakPinWidget.Pin()->GetDefaultValueVisibility();
							}

							if(Visibility == EVisibility::Visible && WeakPropertyBindingWidget.IsValid())
							{
								Visibility = WeakPropertyBindingWidget.Pin()->GetVisibility() == EVisibility::Visible ? EVisibility::Collapsed : EVisibility::Visible;
							}

							return Visibility;
						}));
					}	
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE