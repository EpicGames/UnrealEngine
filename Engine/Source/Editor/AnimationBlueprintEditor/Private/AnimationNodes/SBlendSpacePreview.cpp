// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationNodes/SBlendSpacePreview.h"

#include "AnimGraphNode_Base.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/BlendSpace.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "Engine/Blueprint.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Layout/Children.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Modules/ModuleManager.h"
#include "PersonaModule.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/Object.h"
#include "Widgets/Layout/SBox.h"

class FProperty;

void SBlendSpacePreview::Construct(const FArguments& InArgs, UAnimGraphNode_Base* InNode)
{
	check(InNode);

	Node = InNode;

	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");

	FBlendSpacePreviewArgs Args;

	Args.PreviewBlendSpace = MakeAttributeLambda([this](){ return CachedBlendSpace.Get(); });
	Args.PreviewPosition = MakeAttributeLambda([this]() { return CachedPosition; });
	Args.PreviewFilteredPosition = MakeAttributeLambda([this]() { return CachedFilteredPosition; });
	Args.OnGetBlendSpaceSampleName = InArgs._OnGetBlendSpaceSampleName;

	ChildSlot
	[
		SNew(SBox)
		.MinDesiredHeight_Lambda([this]()
		{
			return 100.0f;
		})
		.Visibility(this, &SBlendSpacePreview::GetBlendSpaceVisibility)
		[
			PersonaModule.CreateBlendSpacePreviewWidget(Args)
		]
	];

	RegisterActiveTimer(1.0f / 60.0f, FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime)
	{
		GetBlendSpaceInfo(CachedBlendSpace, CachedPosition, CachedFilteredPosition);
		return EActiveTimerReturnType::Continue;
	}));
}

EVisibility SBlendSpacePreview::GetBlendSpaceVisibility() const
{
	if (Node.Get() != nullptr)
	{
		if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(Node.Get()))
		{
			if (FProperty* Property = FKismetDebugUtilities::FindClassPropertyForNode(Blueprint, Node.Get()))
			{
				if (UObject* ActiveObject = Blueprint->GetObjectBeingDebugged())
				{
					return EVisibility::Visible;
				}
			}
		}
	}

	return EVisibility::Collapsed;
}

bool SBlendSpacePreview::GetBlendSpaceInfo(TWeakObjectPtr<const UBlendSpace>& OutBlendSpace, FVector& OutPosition, FVector& OutFilteredPosition) const
{
	if (Node.Get() != nullptr)
	{
		if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(Node.Get()))
		{
			if (UObject* ActiveObject = Blueprint->GetObjectBeingDebugged())
			{
				if (UAnimBlueprintGeneratedClass* Class = Cast<UAnimBlueprintGeneratedClass>(ActiveObject->GetClass()))
				{
					if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(Class->ClassGeneratedBy))
					{
						UAnimBlueprint* RootAnimBP = UAnimBlueprint::FindRootAnimBlueprint(AnimBlueprint);
						AnimBlueprint = RootAnimBP ? RootAnimBP : AnimBlueprint;
					
						const FAnimBlueprintDebugData& DebugData = AnimBlueprint->GetAnimBlueprintGeneratedClass()->GetAnimBlueprintDebugData();
						if(const int32* NodeIndexPtr = DebugData.NodePropertyToIndexMap.Find(Node))
						{
							int32 AnimNodeIndex = *NodeIndexPtr;
							// reverse node index temporarily because of a bug in NodeGuidToIndexMap
							AnimNodeIndex = Class->GetAnimNodeProperties().Num() - AnimNodeIndex - 1;

							if (const FAnimBlueprintDebugData::FBlendSpacePlayerRecord* DebugInfo = DebugData.BlendSpacePlayerRecordsThisFrame.FindByPredicate(
							[AnimNodeIndex](const FAnimBlueprintDebugData::FBlendSpacePlayerRecord& InRecord){ return InRecord.NodeID == AnimNodeIndex; }))
							{
								OutBlendSpace = DebugInfo->BlendSpace.Get();
								OutPosition = DebugInfo->Position;
								OutFilteredPosition = DebugInfo->FilteredPosition;
								return true;
							}
						}
					}
				}
			}
		}
	}

	OutBlendSpace = nullptr;
	OutPosition = FVector::ZeroVector;
	OutFilteredPosition = FVector::ZeroVector;
	return false;
}
