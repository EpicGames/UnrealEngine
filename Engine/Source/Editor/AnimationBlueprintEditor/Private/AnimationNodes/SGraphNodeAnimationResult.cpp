// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationNodes/SGraphNodeAnimationResult.h"

#include "AnimGraphNode_Base.h"
#include "GenericPlatform/ICursor.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"

class SWidget;

/////////////////////////////////////////////////////
// SGraphNodeAnimationResult

void SGraphNodeAnimationResult::Construct(const FArguments& InArgs, UAnimGraphNode_Base* InNode)
{
	this->GraphNode = InNode;

	this->SetCursor(EMouseCursor::CardinalCross);

	this->UpdateGraphNode();

	SAnimationGraphNode::Construct(SAnimationGraphNode::FArguments(), InNode);
}

TSharedRef<SWidget> SGraphNodeAnimationResult::CreateNodeContentArea()
{
	return SNew(SBorder)
		.BorderImage( FAppStyle::GetBrush("NoBorder") )
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.Padding( FMargin(0,3) )
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.FillWidth(1.0f)
			[
				// LEFT
				SAssignNew(LeftNodeBox, SVerticalBox)
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image( FAppStyle::GetBrush("Graph.AnimationResultNode.Body") )
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Right)
			[
				// RIGHT
				SAssignNew(RightNodeBox, SVerticalBox)
			]
		];
}
