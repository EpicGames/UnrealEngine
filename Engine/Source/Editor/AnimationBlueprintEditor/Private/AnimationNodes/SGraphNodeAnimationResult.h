// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationNodes/SAnimationGraphNode.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SWidget;

class SGraphNodeAnimationResult : public SAnimationGraphNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeAnimationResult){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, class UAnimGraphNode_Base* InNode);

protected:
	// SGraphNode interface
	virtual TSharedRef<SWidget> CreateNodeContentArea() override;
	// End of SGraphNode interface
};
