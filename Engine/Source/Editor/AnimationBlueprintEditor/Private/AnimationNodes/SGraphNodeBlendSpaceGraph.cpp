// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationNodes/SGraphNodeBlendSpaceGraph.h"

#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_BlendSpaceGraphBase.h"
#include "Animation/BlendSpace.h"
#include "AnimationNodes/SAnimationGraphNode.h"
#include "BlendSpaceGraph.h"
#include "Containers/ArrayView.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "EdGraph/EdGraphNode.h"
#include "Fonts/SlateFontInfo.h"
#include "Framework/Application/SlateApplication.h"
#include "GenericPlatform/GenericApplication.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDocumentation.h"
#include "IDocumentationPage.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "Modules/ModuleManager.h"
#include "PersonaDelegates.h"
#include "PersonaModule.h"
#include "AnimationNodes/SBlendSpacePreview.h"
#include "SLevelOfDetailBranchNode.h"
#include "SNodePanel.h"
#include "SPoseWatchOverlay.h"
#include "SlotBase.h"
#include "Styling/CoreStyle.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "Types/SlateStructs.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Images/SImage.h"

class SWidget;
class UEdGraphPin;

#define LOCTEXT_NAMESPACE "SGraphNodeBlendSpaceGraph"

void SGraphNodeBlendSpaceGraph::Construct(const FArguments& InArgs, UAnimGraphNode_BlendSpaceGraphBase* InNode)
{
	GraphNode = InNode;

	SetCursor(EMouseCursor::CardinalCross);

	FastPathWidget = SNew(SImage)
		.Image(FAppStyle::Get().GetBrush(TEXT("Graph.AnimationFastPathIndicator")))
		.ToolTip(IDocumentation::Get()->CreateToolTip(LOCTEXT("AnimGraphNodeIndicatorTooltip", "Fast path enabled: This node is not using any Blueprint calls to update its data."), nullptr, TEXT("Shared/GraphNodes/Animation"), TEXT("GraphNode_FastPathInfo")))
		.Visibility(EVisibility::Visible);

	PoseWatchWidget = SNew(SPoseWatchOverlay, InNode);

	UpdateGraphNode();

	SAnimationGraphNode::ReconfigurePinWidgetsForPropertyBindings(CastChecked<UAnimGraphNode_Base>(GraphNode), SharedThis(this), [this](UEdGraphPin* InPin){ return FindWidgetForPin(InPin); });
}

UEdGraph* SGraphNodeBlendSpaceGraph::GetInnerGraph() const
{
	UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode = CastChecked<UAnimGraphNode_BlendSpaceGraphBase>(GraphNode);

	return BlendSpaceNode->GetBlendSpaceGraph();
}

TArray<FOverlayWidgetInfo> SGraphNodeBlendSpaceGraph::GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const
{
	TArray<FOverlayWidgetInfo> Widgets;

	if (UAnimGraphNode_Base* AnimNode = CastChecked<UAnimGraphNode_Base>(GraphNode, ECastCheckedType::NullAllowed))
	{
		if (AnimNode->BlueprintUsage == EBlueprintUsage::DoesNotUseBlueprint)
		{
			const FSlateBrush* ImageBrush = FAppStyle::Get().GetBrush(TEXT("Graph.AnimationFastPathIndicator"));

			FOverlayWidgetInfo Info;
			Info.OverlayOffset = FVector2f(WidgetSize.X - (ImageBrush->ImageSize.X * 0.5f), -(ImageBrush->ImageSize.Y * 0.5f));
			Info.Widget = FastPathWidget;

			Widgets.Add(Info);
		}

		if (PoseWatchWidget->IsPoseWatchValid())
		{
			FOverlayWidgetInfo Info;
			Info.OverlayOffset = PoseWatchWidget->GetOverlayOffset();
			Info.Widget = PoseWatchWidget;
			Widgets.Add(Info);
		}
	}

	return Widgets;
}

TSharedPtr<SToolTip> SGraphNodeBlendSpaceGraph::GetComplexTooltip()
{
	if (UBlendSpaceGraph* BlendSpaceGraph = CastChecked<UBlendSpaceGraph>(GetInnerGraph()))
	{
		struct LocalUtils
		{
			static bool IsInteractive()
			{
				const FModifierKeysState ModifierKeys = FSlateApplication::Get().GetModifierKeys();
				return ( ModifierKeys.IsAltDown() && ModifierKeys.IsControlDown() );
			}
		};

		FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");

		FBlendSpacePreviewArgs Args;
		Args.PreviewBlendSpace = BlendSpaceGraph->BlendSpace;

		TSharedPtr<SToolTip> FinalToolTip = nullptr;
		TSharedPtr<SVerticalBox> Container = nullptr;
		SAssignNew(FinalToolTip, SToolTip)
		.IsInteractive_Static(&LocalUtils::IsInteractive)
		[
			SAssignNew(Container, SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew( STextBlock )
				.Text(this, &SGraphNodeBlendSpaceGraph::GetTooltipTextForNode)
				.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
				.WrapTextAt(160.0f)
			]
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				SNew(SBox)
				.WidthOverride(200.0f)
				.HeightOverride(150.0f)
				[
					PersonaModule.CreateBlendSpacePreviewWidget(Args)
				]
			]
		];

		// Check to see whether this node has a documentation excerpt. If it does, create a doc box for the tooltip
		TSharedRef<IDocumentationPage> DocPage = IDocumentation::Get()->GetPage(GraphNode->GetDocumentationLink(), nullptr);
		if(DocPage->HasExcerpt(GraphNode->GetDocumentationExcerptName()))
		{
			Container->AddSlot()
			.AutoHeight()
			.Padding(FMargin( 0.0f, 5.0f ))
			[
				IDocumentation::Get()->CreateToolTip(FText::FromString("Documentation"), nullptr, GraphNode->GetDocumentationLink(), GraphNode->GetDocumentationExcerptName())
			];
		}

		return FinalToolTip;
	}
	else
	{
		return SNew(SToolTip)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew( STextBlock )
					.Text(LOCTEXT("InvalidBlendspaceMessage", "ERROR: Invalid Blendspace"))
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
					.WrapTextAt(160.0f)
				]
			];
	}

}

TSharedRef<SWidget> SGraphNodeBlendSpaceGraph::CreateNodeBody()
{
	TSharedRef<SWidget> NodeBody = SGraphNodeK2Composite::CreateNodeBody();
	
	UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode = CastChecked<UAnimGraphNode_BlendSpaceGraphBase>(GraphNode);

	auto UseLowDetailNode = [this]()
	{
		return GetCurrentLOD() <= EGraphRenderingLOD::LowDetail;
	};
	
	return SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			NodeBody
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(4.0f, 2.0f, 4.0f, 2.0f)
		[
			SAnimationGraphNode::CreateNodeFunctionsWidget(BlendSpaceNode, MakeAttributeLambda(UseLowDetailNode))
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SLevelOfDetailBranchNode)
			.UseLowDetailSlot_Lambda(UseLowDetailNode)
			.LowDetail()
			[
				SNew(SSpacer)
				.Size(FVector2D(100.0f, 100.f))
			]
			.HighDetail()
			[
				SNew(SBlendSpacePreview, CastChecked<UAnimGraphNode_Base>(GraphNode))
				.OnGetBlendSpaceSampleName(FOnGetBlendSpaceSampleName::CreateLambda([this, WeakBlendSpaceNode = TWeakObjectPtr<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceNode)](int32 InSampleIndex) -> FName
				{
					if(WeakBlendSpaceNode.Get())
					{
						UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode = WeakBlendSpaceNode.Get();
						return BlendSpaceNode->GetGraphs()[InSampleIndex]->GetFName();
					}

					return NAME_None;
				}))
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Right)
		.Padding(4.0f, 2.0f, 4.0f, 2.0f)
		[
			SAnimationGraphNode::CreateNodeTagWidget(BlendSpaceNode, MakeAttributeLambda(UseLowDetailNode))
		];
}

#undef LOCTEXT_NAMESPACE