// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "KismetNodes/SGraphNodeK2Composite.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SPoseWatchOverlay;
class SToolTip;
class SWidget;
class UEdGraph;
struct FOverlayWidgetInfo;

class SGraphNodeBlendSpaceGraph : public SGraphNodeK2Composite
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeBlendSpaceGraph){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, class UAnimGraphNode_BlendSpaceGraphBase* InNode);

protected:
	// SGraphNodeK2Composite interface
	virtual UEdGraph* GetInnerGraph() const override;
	virtual TArray<FOverlayWidgetInfo> GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const override;
	virtual TSharedRef<SWidget> CreateNodeBody() override;
	
	// SGraphNode interface
	TSharedPtr<SToolTip> GetComplexTooltip() override;

private:
	TSharedPtr<SWidget> FastPathWidget;

	TSharedPtr<SPoseWatchOverlay> PoseWatchWidget;
};
