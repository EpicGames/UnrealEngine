// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationNodes/SGraphNodeBlendSpacePlayer.h"

#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_BlendSpacePlayer.h"
#include "AnimNodes/AnimNode_BlendSpacePlayer.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimationAsset.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "Engine/Blueprint.h"
#include "GenericPlatform/ICursor.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "AnimationNodes/SBlendSpacePreview.h"
#include "SLevelOfDetailBranchNode.h"
#include "SlotBase.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/UnrealNames.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"

class UObject;

void SGraphNodeBlendSpacePlayer::Construct(const FArguments& InArgs, UAnimGraphNode_Base* InNode)
{
	this->GraphNode = InNode;

	this->SetCursor(EMouseCursor::CardinalCross);

	this->UpdateGraphNode();

	CachedSyncGroupName = NAME_None;

	SAnimationGraphNode::Construct(SAnimationGraphNode::FArguments(), InNode);

	RegisterActiveTimer(1.0f / 60.0f, FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime)
	{
		UpdateGraphSyncLabel();
		return EActiveTimerReturnType::Continue;
	}));
}

void SGraphNodeBlendSpacePlayer::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
{
	SAnimationGraphNode::CreateBelowPinControls(MainBox);

	// Insert above the error reporting bar (but above the tag/functions)
	MainBox->InsertSlot(FMath::Max(0, MainBox->NumSlots() - DebugGridSlotReverseIndex))
	.AutoHeight()
	.VAlign(VAlign_Fill)
	.Padding(0.0f)
	[
		SNew(SLevelOfDetailBranchNode)
		.UseLowDetailSlot(this, &SGraphNodeBlendSpacePlayer::UseLowDetailNodeTitles)
		.LowDetail()
		[
			SNew(SSpacer)
			.Size(FVector2D(100.0f, 100.f))
		]
		.HighDetail()
		[
			SNew(SBlendSpacePreview, CastChecked<UAnimGraphNode_Base>(GraphNode))
		]
	];
}

void SGraphNodeBlendSpacePlayer::UpdateGraphSyncLabel()
{
	if (UAnimGraphNode_BlendSpacePlayer* VisualBlendSpacePlayer = Cast<UAnimGraphNode_BlendSpacePlayer>(GraphNode))
	{
		FName CurrentSyncGroupName = NAME_None;

		if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(GraphNode)))
		{
			if(UAnimBlueprintGeneratedClass* GeneratedClass = AnimBlueprint->GetAnimBlueprintGeneratedClass())
			{
				if (UObject* ActiveObject = AnimBlueprint->GetObjectBeingDebugged())
				{
					if(VisualBlendSpacePlayer->Node.GetGroupMethod() == EAnimSyncMethod::Graph)
					{
						int32 NodeIndex = GeneratedClass->GetNodeIndexFromGuid(VisualBlendSpacePlayer->NodeGuid);
						if(NodeIndex != INDEX_NONE)
						{
							if(const FName* SyncGroupNamePtr = GeneratedClass->GetAnimBlueprintDebugData().NodeSyncsThisFrame.Find(NodeIndex))
							{
								CurrentSyncGroupName = *SyncGroupNamePtr;
							}
						}
					}
				}
			}
		}

		if(CachedSyncGroupName != CurrentSyncGroupName)
		{
			// Invalidate the node title so we can dynamically display the sync group gleaned from the graph
			VisualBlendSpacePlayer->OnNodeTitleChangedEvent().Broadcast();
			CachedSyncGroupName = CurrentSyncGroupName;
		}
	}
}
