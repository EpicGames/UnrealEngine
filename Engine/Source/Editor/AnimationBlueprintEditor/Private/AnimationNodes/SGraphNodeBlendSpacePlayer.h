// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationNodes/SAnimationGraphNode.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SVerticalBox;
class UAnimGraphNode_Base;

class SGraphNodeBlendSpacePlayer : public SAnimationGraphNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeBlendSpacePlayer){}
	SLATE_END_ARGS()

	// Reverse index of the debug grid widget
	static const int32 DebugGridSlotReverseIndex = 2;
	
	void Construct(const FArguments& InArgs, UAnimGraphNode_Base* InNode);

	// SGraphNode interface
	virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;
	// End of SGraphNode interface

protected:
	// Invalidates the node's label if we are syncing based on graph context
	void UpdateGraphSyncLabel();

	// Cached name to display when sync groups are dynamic
	FName CachedSyncGroupName;
};
