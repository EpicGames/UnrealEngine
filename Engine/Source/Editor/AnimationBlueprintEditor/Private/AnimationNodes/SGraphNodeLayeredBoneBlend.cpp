// Copyright Epic Games, Inc. All Rights Reserved.


#include "AnimationNodes/SGraphNodeLayeredBoneBlend.h"

#include "AnimGraphNode_LayeredBoneBlend.h"
#include "GenericPlatform/ICursor.h"
#include "GraphEditorSettings.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Misc/Optional.h"
#include "SlotBase.h"
#include "Types/SlateEnums.h"
#include "Widgets/SBoxPanel.h"

class SWidget;

/////////////////////////////////////////////////////
// SGraphNodeLayeredBoneBlend

void SGraphNodeLayeredBoneBlend::Construct(const FArguments& InArgs, UAnimGraphNode_LayeredBoneBlend* InNode)
{
	this->GraphNode = Node = InNode;

	this->SetCursor(EMouseCursor::CardinalCross);

	this->UpdateGraphNode();

	SAnimationGraphNode::Construct(SAnimationGraphNode::FArguments(), InNode);
}

void SGraphNodeLayeredBoneBlend::CreateInputSideAddButton(TSharedPtr<SVerticalBox> InputBox)
{
	TSharedRef<SWidget> AddPinButton = AddPinButtonContent(
		NSLOCTEXT("LayeredBoneBlendNode", "LayeredBoneBlendNodeAddPinButton", "Add pin"),
		NSLOCTEXT("LayeredBoneBlendNode", "LayeredBoneBlendNodeAddPinButton_Tooltip", "Adds a input pose to the node"),
		false);

	FMargin AddPinPadding = Settings->GetInputPinPadding();
	AddPinPadding.Top += 6.0f;

	InputBox->AddSlot()
		.AutoHeight()
		.VAlign(VAlign_Center)
		.Padding(AddPinPadding)
		[
			AddPinButton
		];
}

FReply SGraphNodeLayeredBoneBlend::OnAddPin()
{
	Node->AddPinToBlendByFilter();

	return FReply::Handled();
}
