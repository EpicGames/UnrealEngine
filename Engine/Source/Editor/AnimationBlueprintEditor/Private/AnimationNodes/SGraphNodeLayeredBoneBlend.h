// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationNodes/SAnimationGraphNode.h"
#include "Input/Reply.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SVerticalBox;
class UAnimGraphNode_LayeredBoneBlend;

class SGraphNodeLayeredBoneBlend : public SAnimationGraphNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeLayeredBoneBlend){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UAnimGraphNode_LayeredBoneBlend* InNode);

protected:
	// SGraphNode interface
	virtual void CreateInputSideAddButton(TSharedPtr<SVerticalBox> InputBox) override;
	virtual FReply OnAddPin() override;
	// End of SGraphNode interface

private:

	// The node that we represent
	UAnimGraphNode_LayeredBoneBlend* Node;

};
