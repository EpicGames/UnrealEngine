// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationNodes/SGraphNodeLinkedLayer.h"

#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_LinkedAnimGraph.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimNode_LinkedAnimGraph.h"
#include "BlueprintEditorModule.h"
#include "Delegates/Delegate.h"
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "Engine/Blueprint.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/UnrealNames.h"

class USkeletalMeshComponent;

void SGraphNodeLinkedLayer::Construct(const FArguments& InArgs, UAnimGraphNode_Base* InNode)
{
	this->GraphNode = InNode;
	this->UpdateGraphNode();

	CachedTargetName = NAME_None;

	SAnimationGraphNode::Construct(SAnimationGraphNode::FArguments(), InNode);

	RegisterActiveTimer(1.0f / 10.0f, FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime)
	{
		UpdateNodeLabel();
		return EActiveTimerReturnType::Continue;
	}));
}

void SGraphNodeLinkedLayer::UpdateNodeLabel()
{
	if (UAnimGraphNode_LinkedAnimGraph* VisualLinkedAnimLayer = Cast<UAnimGraphNode_LinkedAnimGraph>(GraphNode))
	{
		FName FoundName = NAME_None;
		
		if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(GraphNode)))
		{
			if (const UAnimInstance* InstanceBeingDebugged = Cast<const UAnimInstance>(AnimBlueprint->GetObjectBeingDebugged()))
			{
				USkeletalMeshComponent* Component = InstanceBeingDebugged->GetSkelMeshComponent();
				if (Component != nullptr)
				{
					if (const FAnimNode_LinkedAnimGraph* AnimGraphNode = static_cast<FAnimNode_LinkedAnimGraph*>(VisualLinkedAnimLayer->FindDebugAnimNode(Component)))
					{
						if (const UAnimInstance* TargetInstance = AnimGraphNode->GetTargetInstance<UAnimInstance>())
						{
							FoundName = TargetInstance->GetFName();

							if (FoundName != CachedTargetName)
							{
								VisualLinkedAnimLayer->OnNodeTitleChangedEvent().Broadcast();
								CachedTargetName = FoundName;
								if (IAssetEditorInstance* EditorInstance = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->FindEditorForAsset(AnimBlueprint, false))
								{
									static_cast<IBlueprintEditor*>(EditorInstance)->RefreshMyBlueprint();						
								}
							}
						}
					}
				}
			}	
		}
	}
}
