// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationNodes/SAnimationGraphNode.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UAnimGraphNode_Base;

class SGraphNodeLinkedLayer : public SAnimationGraphNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeLinkedLayer){}
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs, UAnimGraphNode_Base* InNode);

protected:
	void UpdateNodeLabel();

	FName CachedTargetName;
};
