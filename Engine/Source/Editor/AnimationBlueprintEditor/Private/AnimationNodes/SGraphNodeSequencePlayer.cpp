// Copyright Epic Games, Inc. All Rights Reserved.


#include "AnimationNodes/SGraphNodeSequencePlayer.h"

#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_SequencePlayer.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimNode_SequencePlayer.h"
#include "Animation/AnimSequenceBase.h"
#include "Animation/AnimationAsset.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "Delegates/Delegate.h"
#include "Engine/Blueprint.h"
#include "GenericPlatform/ICursor.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/KismetDebugUtilities.h"
#include "Layout/Margin.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SGraphNode.h"
#include "SLevelOfDetailBranchNode.h"
#include "SNodePanel.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Styling/SlateTypes.h"
#include "Styling/StyleColors.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "Types/WidgetActiveTimerDelegate.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"

class FProperty;

/////////////////////////////////////////////////////
// SGraphNodeSequencePlayer

void SGraphNodeSequencePlayer::Construct(const FArguments& InArgs, UAnimGraphNode_Base* InNode)
{
	this->GraphNode = InNode;

	this->SetCursor(EMouseCursor::CardinalCross);

	this->UpdateGraphNode();

	CachedSyncGroupName = NAME_None;

	SAnimationGraphNode::Construct(SAnimationGraphNode::FArguments(), InNode);

	RegisterActiveTimer(1.0f / 60.0f, FWidgetActiveTimerDelegate::CreateLambda([this](double InCurrentTime, float InDeltaTime)
	{
		UpdateGraphSyncLabel();
		return EActiveTimerReturnType::Continue;
	}));
}

void SGraphNodeSequencePlayer::GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const
{
	SGraphNodeK2Base::GetNodeInfoPopups(Context, Popups);

	UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(GraphNode));
	if(AnimBlueprint)
	{
		UAnimInstance* ActiveObject = Cast<UAnimInstance>(AnimBlueprint->GetObjectBeingDebugged());
		UAnimBlueprintGeneratedClass* Class = AnimBlueprint->GetAnimBlueprintGeneratedClass();

		const FLinearColor Color(1.f, 0.5f, 0.25f);

		// Display various types of debug data
		if ((ActiveObject != NULL) && (Class != NULL))
		{
			if (Class->GetAnimNodeProperties().Num())
			{
				if(int32* NodeIndexPtr = Class->GetAnimBlueprintDebugData().NodePropertyToIndexMap.Find(TWeakObjectPtr<UAnimGraphNode_Base>(Cast<UAnimGraphNode_Base>(GraphNode))))
				{
					int32 AnimNodeIndex = *NodeIndexPtr;
					// reverse node index temporarily because of a bug in NodeGuidToIndexMap
					AnimNodeIndex = Class->GetAnimNodeProperties().Num() - AnimNodeIndex - 1;

					FString PopupText;
					for(auto & NodeValue : Class->GetAnimBlueprintDebugData().NodeValuesThisFrame)
					{
						if (NodeValue.NodeID == AnimNodeIndex && NodeValue.Text.Contains("Asset ="))
						{
							if (PopupText.IsEmpty())
							{
								PopupText = NodeValue.Text;
							}
							else
							{
								PopupText = FString::Format(TEXT("{0}\n{1}"), {PopupText, NodeValue.Text});
							}
						}
					}
					if (!PopupText.IsEmpty())
					{
						Popups.Emplace(nullptr, Color, PopupText);
					}
				}
			}
		}
	}
}

FText SGraphNodeSequencePlayer::GetPositionTooltip() const
{
	float Position;
	float Length;
	int32 FrameCount;
	if (GetSequencePositionInfo(/*out*/ Position, /*out*/ Length, /*out*/ FrameCount))
	{
		const int32 Minutes = FMath::TruncToInt(Position/60.0f);
		const int32 Seconds = FMath::TruncToInt(Position) % 60;
		const int32 Hundredths = FMath::TruncToInt(FMath::Fractional(Position)*100);

		FString MinuteStr;
		if (Minutes > 0)
		{
			MinuteStr = FString::Printf(TEXT("%dm"), Minutes);
		}

		const FString SecondStr = FString::Printf(TEXT("%02ds"), Seconds);

		const FString HundredthsStr = FString::Printf(TEXT(".%02d"), Hundredths);

		const int32 CurrentFrame = FMath::TruncToInt((Position / Length) * FrameCount);

		const FString FramesStr = FString::Printf(TEXT("Frame %d"), CurrentFrame);

		return FText::FromString(FString::Printf(TEXT("%s (%s%s%s)"), *FramesStr, *MinuteStr, *SecondStr, *HundredthsStr));
	}

	return NSLOCTEXT("SGraphNodeSequencePlayer", "PositionToolTip_Default", "Position");
}

void SGraphNodeSequencePlayer::UpdateGraphNode()
{
	SGraphNode::UpdateGraphNode();
}

void SGraphNodeSequencePlayer::CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox)
{
	SAnimationGraphNode::CreateBelowPinControls(MainBox);

	auto UseLowDetailNode = [this]()
	{
		return GetCurrentLOD() <= EGraphRenderingLOD::LowDetail;
	};
	
	// Insert above the error reporting bar (but above the tag/functions)
	MainBox->InsertSlot(FMath::Max(0, MainBox->NumSlots() - DebugSliderSlotReverseIndex))
	.AutoHeight()
	.VAlign( VAlign_Fill )
	.Padding(FMargin(4.0f, 0.0f))
	[
		SNew(SLevelOfDetailBranchNode)
		.UseLowDetailSlot_Lambda(UseLowDetailNode)
		.LowDetail()
		[
			SNew(SSpacer)
			.Size(FVector2D(16.0f, 16.f))
		]
		.HighDetail()
		[
			SNew(SSlider)
			.Style(&FAppStyle::Get().GetWidgetStyle<FSliderStyle>("AnimBlueprint.AssetPlayerSlider"))
			.ToolTipText(this, &SGraphNodeSequencePlayer::GetPositionTooltip)
			.Visibility(this, &SGraphNodeSequencePlayer::GetSliderVisibility)
			.Value(this, &SGraphNodeSequencePlayer::GetSequencePositionRatio)
			.OnValueChanged(this, &SGraphNodeSequencePlayer::SetSequencePositionRatio)
			.Locked(false)
			.SliderHandleColor(FStyleColors::White)
			.SliderBarColor(FStyleColors::Foreground)
		]
	];
}

FAnimNode_SequencePlayer* SGraphNodeSequencePlayer::GetSequencePlayer() const
{
	if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(GraphNode))
	{
		if (UObject* ActiveObject = Blueprint->GetObjectBeingDebugged())
		{
			if (UAnimGraphNode_SequencePlayer* VisualSequencePlayer = Cast<UAnimGraphNode_SequencePlayer>(GraphNode))
			{
				if (UAnimBlueprintGeneratedClass* Class = Cast<UAnimBlueprintGeneratedClass>((UObject*)ActiveObject->GetClass()))
				{
					return Class->GetPropertyInstance<FAnimNode_SequencePlayer>(ActiveObject, VisualSequencePlayer);
				}
			}
		}
	}
	return NULL;
}

EVisibility SGraphNodeSequencePlayer::GetSliderVisibility() const
{
	if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(GraphNode))
	{
		if (FProperty* Property = FKismetDebugUtilities::FindClassPropertyForNode(Blueprint, GraphNode))
		{
			if (UObject* ActiveObject = Blueprint->GetObjectBeingDebugged())
			{
				return EVisibility::Visible;
			}
		}
	}

	return EVisibility::Collapsed;
}

bool SGraphNodeSequencePlayer::GetSequencePositionInfo(float& Out_Position, float& Out_Length, int32& Out_FrameCount) const
{
	if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(GraphNode))
	{
		if (UObject* ActiveObject = Blueprint->GetObjectBeingDebugged())
		{
			if (UAnimGraphNode_SequencePlayer* VisualSequencePlayer = Cast<UAnimGraphNode_SequencePlayer>(GraphNode))
			{
				if (UAnimBlueprintGeneratedClass* Class = Cast<UAnimBlueprintGeneratedClass>((UObject*)ActiveObject->GetClass()))
				{
					if(int32* NodeIndexPtr = Class->GetAnimBlueprintDebugData().NodePropertyToIndexMap.Find(TWeakObjectPtr<UAnimGraphNode_Base>(Cast<UAnimGraphNode_Base>(GraphNode))))
					{
						int32 AnimNodeIndex = *NodeIndexPtr;
						// reverse node index temporarily because of a bug in NodeGuidToIndexMap
						AnimNodeIndex = Class->GetAnimNodeProperties().Num() - AnimNodeIndex - 1;

						if (FAnimBlueprintDebugData::FSequencePlayerRecord* DebugInfo = Class->GetAnimBlueprintDebugData().SequencePlayerRecordsThisFrame.FindByPredicate([AnimNodeIndex](const FAnimBlueprintDebugData::FSequencePlayerRecord& InRecord){ return InRecord.NodeID == AnimNodeIndex; }))
						{
							Out_Position = DebugInfo->Position;
							Out_Length = DebugInfo->Length;
							Out_FrameCount = DebugInfo->FrameCount;

							return true;
						}
					}
				}
			}
		}
	}

	Out_Position = 0.0f;
	Out_Length = 0.0f;
	Out_FrameCount = 0;
	return false;
}

float SGraphNodeSequencePlayer::GetSequencePositionRatio() const
{
	float Position;
	float Length;
	int32 FrameCount;
	if (GetSequencePositionInfo(/*out*/ Position, /*out*/ Length, /*out*/ FrameCount))
	{
		return Position / Length;
	}
	return 0.0f;
}

void SGraphNodeSequencePlayer::SetSequencePositionRatio(float NewRatio)
{
	if(FAnimNode_SequencePlayer* SequencePlayer = GetSequencePlayer())
	{
		UAnimSequenceBase* Sequence = SequencePlayer->GetSequence();
		if (Sequence != NULL)
		{
			const float NewTime = NewRatio * Sequence->GetPlayLength();
			SequencePlayer->SetAccumulatedTime(NewTime);
		}
	}
}

void SGraphNodeSequencePlayer::UpdateGraphSyncLabel()
{
	if (UAnimGraphNode_SequencePlayer* VisualSequencePlayer = Cast<UAnimGraphNode_SequencePlayer>(GraphNode))
	{
		FName CurrentSyncGroupName = NAME_None;

		if (UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(GraphNode)))
		{
			if(UAnimBlueprintGeneratedClass* GeneratedClass = AnimBlueprint->GetAnimBlueprintGeneratedClass())
			{
				if (UObject* ActiveObject = AnimBlueprint->GetObjectBeingDebugged())
				{
					if(VisualSequencePlayer->Node.GetGroupMethod() == EAnimSyncMethod::Graph)
					{
						int32 NodeIndex = GeneratedClass->GetNodeIndexFromGuid(VisualSequencePlayer->NodeGuid);
						if(NodeIndex != INDEX_NONE)
						{
							if(const FName* SyncGroupNamePtr = GeneratedClass->GetAnimBlueprintDebugData().NodeSyncsThisFrame.Find(NodeIndex))
							{
								CurrentSyncGroupName = *SyncGroupNamePtr;
							}
						}
					}
				}
			}
		}

		if(CachedSyncGroupName != CurrentSyncGroupName)
		{
			// Invalidate the node title so we can dynamically display the sync group gleaned from the graph
			VisualSequencePlayer->OnNodeTitleChangedEvent().Broadcast();
			CachedSyncGroupName = CurrentSyncGroupName;
		}
	}
}