// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AnimationNodes/SAnimationGraphNode.h"
#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SVerticalBox;
class UAnimGraphNode_Base;
struct FGraphInformationPopupInfo;
struct FNodeInfoContext;

class SGraphNodeSequencePlayer : public SAnimationGraphNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeSequencePlayer){}
	SLATE_END_ARGS()

	// Reverse index of the debug slider widget
	static const int32 DebugSliderSlotReverseIndex = 2;

	void Construct(const FArguments& InArgs, UAnimGraphNode_Base* InNode);

	// SNodePanel::SNode interface
	virtual void GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const override;
	// End of SNodePanel::SNode interface

	// SGraphNode interface
	virtual void UpdateGraphNode() override;
	virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;
	// End of SGraphNode interface

protected:
	/** Get the sequence player associated with this graph node */
	struct FAnimNode_SequencePlayer* GetSequencePlayer() const;
	EVisibility GetSliderVisibility() const;
	float GetSequencePositionRatio() const;
	void SetSequencePositionRatio(float NewRatio);

	FText GetPositionTooltip() const;

	bool GetSequencePositionInfo(float& Out_Position, float& Out_Length, int32& FrameCount) const;

	// Invalidates the node's label if we are syncing based on graph context
	void UpdateGraphSyncLabel();

	// Cached name to display when sync groups are dynamic
	FName CachedSyncGroupName;
};
