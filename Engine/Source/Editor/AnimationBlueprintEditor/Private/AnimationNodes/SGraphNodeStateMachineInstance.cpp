// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationNodes/SGraphNodeStateMachineInstance.h"

#include "AnimationNodes/SAnimationGraphNode.h"
#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_StateMachineBase.h"
#include "AnimationStateMachineGraph.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/PlatformCrt.h"
#include "Misc/Optional.h"
#include "SNodePanel.h"
#include "SPoseWatchOverlay.h"
#include "Templates/Casts.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"

#define LOCTEXT_NAMESPACE "SGraphNodeStateMachineInstance"

/////////////////////////////////////////////////////
// SGraphNodeStateMachineInstance

void SGraphNodeStateMachineInstance::Construct(const FArguments& InArgs, UAnimGraphNode_StateMachineBase* InNode)
{
	GraphNode = InNode;

	SetCursor(EMouseCursor::CardinalCross);

	PoseWatchWidget = SNew(SPoseWatchOverlay, InNode);

	UpdateGraphNode();
}

UEdGraph* SGraphNodeStateMachineInstance::GetInnerGraph() const
{
	UAnimGraphNode_StateMachineBase* StateMachineInstance = CastChecked<UAnimGraphNode_StateMachineBase>(GraphNode);

	return StateMachineInstance->EditorStateMachineGraph;
}

TArray<FOverlayWidgetInfo> SGraphNodeStateMachineInstance::GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const
{
	TArray<FOverlayWidgetInfo> Widgets;

	if (UAnimGraphNode_Base* AnimNode = CastChecked<UAnimGraphNode_Base>(GraphNode, ECastCheckedType::NullAllowed))
	{
		if (PoseWatchWidget->IsPoseWatchValid())
		{
			FOverlayWidgetInfo Info;
			Info.OverlayOffset = PoseWatchWidget->GetOverlayOffset();
			Info.Widget = PoseWatchWidget;
			Widgets.Add(Info);
		}
	}

	return Widgets;
}

TSharedRef<SWidget> SGraphNodeStateMachineInstance::CreateNodeBody()
{
	TSharedRef<SWidget> NodeBody = SGraphNodeK2Composite::CreateNodeBody();

	UAnimGraphNode_StateMachineBase* StateMachineNode = CastChecked<UAnimGraphNode_StateMachineBase>(GraphNode);

	auto UseLowDetailNode = [this]()
	{
		return GetCurrentLOD() <= EGraphRenderingLOD::LowDetail;
	};

	return SNew(SVerticalBox)
		+ SVerticalBox::Slot()
			.AutoHeight()
			[
				NodeBody
			]
		+ SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Right)
			.Padding(4.0f, 2.0f, 4.0f, 2.0f)
			[
				SAnimationGraphNode::CreateNodeTagWidget(StateMachineNode, MakeAttributeLambda(UseLowDetailNode))
			];
}

#undef LOCTEXT_NAMESPACE