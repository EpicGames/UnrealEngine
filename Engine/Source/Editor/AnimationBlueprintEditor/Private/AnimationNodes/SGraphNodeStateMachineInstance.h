// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "KismetNodes/SGraphNodeK2Composite.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SPoseWatchOverlay;
class UEdGraph;
struct FOverlayWidgetInfo;

class SGraphNodeStateMachineInstance : public SGraphNodeK2Composite
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeStateMachineInstance){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, class UAnimGraphNode_StateMachineBase* InNode);

protected:
	// SGraphNodeK2Composite interface
	virtual UEdGraph* GetInnerGraph() const override;
	virtual TArray<FOverlayWidgetInfo> GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const override;
	virtual TSharedRef<SWidget> CreateNodeBody() override;
	// End of SGraphNodeK2Composite interface

private:
	TSharedPtr<SPoseWatchOverlay> PoseWatchWidget;
};
