// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPoseWatchOverlay.h"

#include "AnimationEditorUtils.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "Engine/PoseWatch.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Math/Color.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateBrush.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"

#define LOCTEXT_NAMESPACE "SPoseWatchOverlay"

const FSlateBrush* SPoseWatchOverlay::IconVisible = nullptr;
const FSlateBrush* SPoseWatchOverlay::IconNotVisible = nullptr;

void SPoseWatchOverlay::Construct(const FArguments& InArgs, UEdGraphNode* InNode)
{
	IconVisible = FAppStyle::GetBrush("Level.VisibleIcon16x");
	IconNotVisible = FAppStyle::GetBrush("Level.NotVisibleIcon16x");

	GraphNode = InNode;

	PoseWatch = AnimationEditorUtils::FindPoseWatchForNode(InNode);
	AnimationEditorUtils::OnPoseWatchesChanged().AddSP(this, &SPoseWatchOverlay::HandlePoseWatchesChanged);

	ChildSlot
	[
		SNew(SButton)
		.ToolTipText(LOCTEXT("TogglePoseWatchVisibility", "Click to toggle visibility"))
		.OnClicked(this, &SPoseWatchOverlay::TogglePoseWatchVisibility)
		.ButtonColorAndOpacity(this, &SPoseWatchOverlay::GetPoseViewColor)
		[
			SNew(SImage).Image(this, &SPoseWatchOverlay::GetPoseViewIcon)
		]
	];
}

void SPoseWatchOverlay::HandlePoseWatchesChanged(UAnimBlueprint* InAnimBlueprint, UEdGraphNode* InNode)
{
	PoseWatch = AnimationEditorUtils::FindPoseWatchForNode(GraphNode.Get());
}

FSlateColor SPoseWatchOverlay::GetPoseViewColor() const
{
	static constexpr float AlphaTemporary = 0.5f;
	static constexpr float AlphaPermanent = 0.9f;

	UPoseWatch* CurPoseWatch = PoseWatch.Get();
	if (CurPoseWatch)
	{
		TObjectPtr<UPoseWatchPoseElement> PoseElement = CurPoseWatch->GetFirstElementOfType<UPoseWatchPoseElement>();
		if (PoseElement)
		{
			FLinearColor OutColor = PoseElement->GetColor();
			OutColor.A = CurPoseWatch->GetShouldDeleteOnDeselect() ? AlphaTemporary : AlphaPermanent;
			return FSlateColor(OutColor);
		}
	}
	return FSlateColor(FColor::Black);
}

const FSlateBrush* SPoseWatchOverlay::GetPoseViewIcon() const
{
	return (PoseWatch.IsValid() && PoseWatch->GetIsVisible()) ? IconVisible : IconNotVisible;
}

FReply SPoseWatchOverlay::TogglePoseWatchVisibility()
{
	if (PoseWatch.IsValid())
	{
		PoseWatch->ToggleIsVisible();
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FVector2D SPoseWatchOverlay::GetOverlayOffset() const
{
	return FVector2D(0 - (IconVisible->ImageSize.X * 0.5f), -(IconVisible->ImageSize.Y * 0.5f));
}

bool SPoseWatchOverlay::IsPoseWatchValid() const
{
	return PoseWatch.IsValid();
}



#undef LOCTEXT_NAMESPACE