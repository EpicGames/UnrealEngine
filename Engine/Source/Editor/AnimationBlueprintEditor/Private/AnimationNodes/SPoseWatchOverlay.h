// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Input/Reply.h"
#include "Math/Vector2D.h"
#include "Styling/SlateColor.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class UAnimBlueprint;
class UEdGraphNode;
struct FSlateBrush;

class SPoseWatchOverlay : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPoseWatchOverlay) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphNode* InNode);
	FVector2D GetOverlayOffset() const;
	bool IsPoseWatchValid() const;

private:
	void HandlePoseWatchesChanged(UAnimBlueprint* InAnimBlueprint, UEdGraphNode* InNode);

	FSlateColor GetPoseViewColor() const;
	const FSlateBrush* GetPoseViewIcon() const;
	FReply TogglePoseWatchVisibility();

	TWeakObjectPtr<UEdGraphNode> GraphNode;
	TWeakObjectPtr<class UPoseWatch> PoseWatch;

	static const FSlateBrush* IconVisible;
	static const FSlateBrush* IconNotVisible;
};