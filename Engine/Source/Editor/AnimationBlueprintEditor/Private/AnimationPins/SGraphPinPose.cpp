// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationPins/SGraphPinPose.h"

#include "Algo/Sort.h"
#include "AnimGraphAttributes.h"
#include "AnimGraphNode_Base.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/Blueprint.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/Children.h"
#include "Layout/Visibility.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/Attribute.h"
#include "SGraphNode.h"
#include "SGraphPanel.h"
#include "SNodePanel.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SWrapBox.h"
#include "Widgets/SCompoundWidget.h"

struct FGeometry;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "SGraphPinPose"

/////////////////////////////////////////////////////
// SGraphPinPose

void SGraphPinPose::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InPin);

	CachedImg_Pin_ConnectedHovered = FAppStyle::GetBrush(TEXT("Graph.PosePin.ConnectedHovered"));
	CachedImg_Pin_Connected = FAppStyle::GetBrush(TEXT("Graph.PosePin.Connected"));
	CachedImg_Pin_DisconnectedHovered = FAppStyle::GetBrush(TEXT("Graph.PosePin.DisconnectedHovered"));
	CachedImg_Pin_Disconnected = FAppStyle::GetBrush(TEXT("Graph.PosePin.Disconnected"));

	ReconfigureWidgetForAttributes();
}

const FSlateBrush* SGraphPinPose::GetPinIcon() const
{
	const FSlateBrush* Brush = NULL;

	if (IsConnected())
	{
		Brush = IsHovered() ? CachedImg_Pin_ConnectedHovered : CachedImg_Pin_Connected;
	}
	else
	{
		Brush = IsHovered() ? CachedImg_Pin_DisconnectedHovered : CachedImg_Pin_Disconnected;
	}

	return Brush;
}


enum class EAttributeUsage
{
	// Attribute is input/output by the pin, but is not currently connected to a producer/consumer
	Unused,

	// Attribute is input/output by the pin and is currently connected to a producer/consumer
	Used,

	// Attribute is not an input/output of the pin but passes through the node
	Passthrough,
};

class SAttributeIndicator : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAttributeIndicator) {}

	SLATE_ARGUMENT(FName, Attribute)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UAnimBlueprint* InAnimBlueprint, const FAnimGraphAttributeDesc* InAttributeDesc, const UEdGraphPin* InPin, EAttributeUsage InUsage, const UAnimGraphNode_Base* InAnimGraphNode)
	{
		AnimBlueprint = InAnimBlueprint;
		AttributeDesc = InAttributeDesc;
		Pin = InPin;
		Usage = InUsage;
		AnimGraphNode = InAnimGraphNode;
		ActiveColor = AttributeDesc->Color.GetSpecifiedColor();
		Color = ActiveColor;
		Value = 0.0f;

		ChildSlot
		[
			SNew(SImage)
			.Visibility_Lambda([this](){ return Value == 0.0f ? EVisibility::Hidden : EVisibility::Visible; })
			.Image(&AttributeDesc->Icon)
			.ColorAndOpacity_Lambda([this](){ return Color; })
		];
	}

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override
	{
		bool bActive = false;

		if((Usage == EAttributeUsage::Used || AttributeDesc->Blend == EAnimGraphAttributeBlend::Blendable) && AnimBlueprint && AnimBlueprint->GetObjectBeingDebugged() != nullptr)
		{
			UAnimBlueprintGeneratedClass* AnimBlueprintClass = (UAnimBlueprintGeneratedClass*)(*(AnimBlueprint->GeneratedClass));
			int32 SourceNodeId = AnimBlueprintClass->GetNodeIndexFromGuid(AnimGraphNode->NodeGuid);
			if(SourceNodeId != INDEX_NONE) 
			{
				const TArray<FAnimBlueprintDebugData::FAttributeRecord>* LinkAttributes;
				if(Pin->Direction == EGPD_Input)
				{
					LinkAttributes = AnimBlueprintClass->GetAnimBlueprintDebugData().NodeInputAttributesThisFrame.Find(SourceNodeId);
				}
				else
				{
					LinkAttributes = AnimBlueprintClass->GetAnimBlueprintDebugData().NodeOutputAttributesThisFrame.Find(SourceNodeId);
				}

				bActive = LinkAttributes && LinkAttributes->ContainsByPredicate([this](const FAnimBlueprintDebugData::FAttributeRecord& InRecord){ return InRecord.Attribute == AttributeDesc->Name; });
			}
		}

		if(bActive)
		{
			Value = 1.0f;
		}
		else
		{
			Value = FMath::FInterpTo(Value, 0.0f, InDeltaTime, 4.0f);
		}

		Color = FMath::Lerp(FLinearColor::Transparent, ActiveColor, Value);
	}

	UAnimBlueprint* AnimBlueprint;
	const FAnimGraphAttributeDesc* AttributeDesc;
	const UEdGraphPin* Pin;
	EAttributeUsage Usage;
	const UAnimGraphNode_Base* AnimGraphNode;
	FLinearColor ActiveColor;
	FLinearColor Color;
	float Value;
};

void SGraphPinPose::ReconfigureWidgetForAttributes()
{
	AttributeInfos.Empty();

	if(UAnimGraphNode_Base* AnimGraphNode = Cast<UAnimGraphNode_Base>(GraphPinObj->GetOwningNode()))
	{
		if(UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(AnimGraphNode)))
		{

			auto AddAttributes = [this, AnimBlueprint, AnimGraphNode](TArrayView<const FName> InUsedAttributes, TArrayView<const FName> InUnusedAttributes, TArrayView<const FName> InPassThroughAttributes)
			{
				auto AddAttribute = [this, AnimBlueprint, AnimGraphNode](FName InAttribute, EAttributeUsage InUsage)
				{
					const FAnimGraphAttributeDesc* AttributeDesc = GetDefault<UAnimGraphAttributes>()->FindAttributeDesc(InAttribute);

					// Early out if we don't want to display this attribute
					if(AttributeDesc)
					{
						if(AttributeDesc->DisplayMode == EAnimGraphAttributesDisplayMode::HideOnPins)
						{
							return;
						}

						// Only store cached attributes for wires on used pins
						if(InUsage != EAttributeUsage::Unused)
						{
							AttributeInfos.Emplace(InAttribute, AttributeDesc->Color.GetSpecifiedColor(), AttributeDesc->Blend, AttributeDesc->SortOrder);

							// Skip displaying passthrough attributes on pins
							if(InUsage != EAttributeUsage::Passthrough && AnimGraphNode->ShouldShowAttributesOnPins())
							{
								const UAnimGraphNode_Base* ProxyGraphNode = AnimGraphNode->GetProxyNodeForAttributes();

								GetLabelAndValue()->AddSlot()
								.Padding(2.0f, 0.0f, 0.0f, 0.0f)
								[
									SNew(SAttributeIndicator, AnimBlueprint, AttributeDesc, GraphPinObj, InUsage, ProxyGraphNode)
								];
							}
						}
					}
				};

				for(const FName& Attribute : InUsedAttributes)
				{
					AddAttribute(Attribute, EAttributeUsage::Used);
				}

				for(const FName& Attribute : InUnusedAttributes)
				{
					AddAttribute(Attribute, EAttributeUsage::Unused);
				}

				for(const FName& Attribute : InPassThroughAttributes)
				{
					AddAttribute(Attribute, EAttributeUsage::Passthrough);
				}
			};

			UAnimBlueprintGeneratedClass* AnimBlueprintClass = (UAnimBlueprintGeneratedClass*)(*(AnimBlueprint->GeneratedClass));

			UAnimGraphNode_Base::FNodeAttributeArray PinAttributes;
			switch(GraphPinObj->Direction)
			{
			case EGPD_Input:
				AnimGraphNode->GetInputLinkAttributes(PinAttributes);
				break;
			case EGPD_Output:
				AnimGraphNode->GetOutputLinkAttributes(PinAttributes);
				break;
			}

			UAnimGraphNode_Base::FNodeAttributeArray NodeAttributes;
			AnimGraphNode->GetInputLinkAttributes(NodeAttributes);
			AnimGraphNode->GetOutputLinkAttributes(NodeAttributes);

			// Unlinked pins display attributes if they are inputs and the node takes them as inputs
			if(GraphPinObj->LinkedTo.Num() == 0 || AnimBlueprintClass == nullptr)
			{
				AddAttributes(TArrayView<const FName>(), PinAttributes, TArrayView<const FName>());
			}
			else if(GraphPinObj->LinkedTo.Num() > 0)
			{
				TArrayView<const FName> CompilerGeneratedAttributes = AnimBlueprintClass->GetAnimBlueprintDebugData().GetNodeAttributes(AnimGraphNode);
				UAnimGraphNode_Base::FNodeAttributeArray UsedAttributes;
				UAnimGraphNode_Base::FNodeAttributeArray UnusedAttributes;
				if(PinAttributes.Num() > 0)
				{
					for(const FName& Name : PinAttributes)
					{
						if(CompilerGeneratedAttributes.Contains(Name))
						{
							UsedAttributes.Add(Name);
						}
						else
						{
							UnusedAttributes.Add(Name);
						}
					}
				}

				UAnimGraphNode_Base::FNodeAttributeArray PassthroughAttributes;
				for(const FName& Name : CompilerGeneratedAttributes)
				{
					if(!PinAttributes.Contains(Name))
					{
						PassthroughAttributes.Add(Name);
					}
				}

				AddAttributes(UsedAttributes, UnusedAttributes, PassthroughAttributes);
			}

			// sort pin attributes
			Algo::Sort(AttributeInfos, [](const FAttributeInfo& InValue0, const FAttributeInfo& InValue1)
			{
				return InValue0.SortOrder < InValue1.SortOrder;
			});

			// overrides attribute set in the base class Construct()
			SetToolTipText(MakeAttributeSP(this, &SGraphPinPose::GetAttributeTooltipText));
		}
	}
}

TArrayView<const SGraphPinPose::FAttributeInfo> SGraphPinPose::GetAttributeInfo() const
{
	SGraphNode* MyOwnerNode = OwnerNodePtr.Pin().Get();
	if (MyOwnerNode && MyOwnerNode->GetOwnerPanel().IsValid())
	{
		if(MyOwnerNode->GetOwnerPanel()->GetCurrentLOD() <= EGraphRenderingLOD::LowDetail)
		{
			return TArrayView<const FAttributeInfo>();
		}
	}

	return AttributeInfos;
}

float SGraphPinPose::GetZoomAmount() const
{
	SGraphNode* MyOwnerNode = OwnerNodePtr.Pin().Get();
	if (MyOwnerNode && MyOwnerNode->GetOwnerPanel().IsValid())
	{
		return MyOwnerNode->GetOwnerPanel()->GetZoomAmount();
	}

	return 1.0f;
}

FText SGraphPinPose::GetAttributeTooltipText() const
{
	if(GraphPinObj)
	{
		UEdGraphPin* OtherPin = GraphPinObj->LinkedTo.Num() > 0 ? GraphPinObj->LinkedTo[0] : nullptr;
		if(OtherPin)
		{
			UAnimGraphNode_Base* Node1 = Cast<UAnimGraphNode_Base>(GraphPinObj->GetOwningNode());
			UAnimGraphNode_Base* Node2 = Cast<UAnimGraphNode_Base>(OtherPin->GetOwningNode());
			if(Node1 && Node2)
			{
				if(UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(Node1)))
				{
					if(AnimBlueprint->GetObjectBeingDebugged() != nullptr)
					{
						UAnimBlueprintGeneratedClass* AnimBlueprintClass = (UAnimBlueprintGeneratedClass*)(*(AnimBlueprint->GeneratedClass));
						int32 SourceNodeId = AnimBlueprintClass->GetNodeIndexFromGuid(Node1->NodeGuid);
						int32 TargetNodeId = AnimBlueprintClass->GetNodeIndexFromGuid(Node2->NodeGuid);

						if(AttributeInfos.Num() > 0)
						{
							FTextBuilder TextBuilder;
							TextBuilder.AppendLine(SGraphPin::GetTooltipText());

							bool bAddedAttributeSubtitle = false;

							for(const SGraphPinPose::FAttributeInfo& AttributeInfo : AttributeInfos)
							{


								if(SourceNodeId != INDEX_NONE && TargetNodeId != INDEX_NONE)
								{
									const TArray<FAnimBlueprintDebugData::FAttributeRecord>* LinkAttributes;

									if(GraphPinObj->Direction == EGPD_Input)
									{
										LinkAttributes = AnimBlueprintClass->GetAnimBlueprintDebugData().NodeInputAttributesThisFrame.Find(SourceNodeId);
									}
									else
									{
										LinkAttributes = AnimBlueprintClass->GetAnimBlueprintDebugData().NodeOutputAttributesThisFrame.Find(SourceNodeId);
									}

									const bool bAttributeUsedInLink = LinkAttributes && LinkAttributes->ContainsByPredicate(
										[&AttributeInfo, TargetNodeId](const FAnimBlueprintDebugData::FAttributeRecord& InRecord)
										{
											return InRecord.Attribute == AttributeInfo.Attribute && InRecord.OtherNode == TargetNodeId; 
										});

									if(bAttributeUsedInLink)
									{
										if(!bAddedAttributeSubtitle)
										{
											TextBuilder.AppendLine(LOCTEXT("AttributesSubtitle", "Attributes:"));
											bAddedAttributeSubtitle = true;
										}

										TextBuilder.AppendLine(AttributeInfo.Attribute);
									}
								}
							}

							return TextBuilder.ToText();
						}
					}
				}
			}
		}
	}
	
	return SGraphPin::GetTooltipText();
}

#undef LOCTEXT_NAMESPACE