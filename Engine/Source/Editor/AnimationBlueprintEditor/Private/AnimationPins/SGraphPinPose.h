// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "SGraphPin.h"
#include "UObject/NameTypes.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class UEdGraphPin;
struct FSlateBrush;

enum class EAnimGraphAttributeBlend;

class SGraphPinPose : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SGraphPinPose)	{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InPin);

	// Struct used by connection drawing to draw attributes
	struct FAttributeInfo
	{
		FAttributeInfo(FName InAttribute, const FLinearColor& InColor, EAnimGraphAttributeBlend InBlend, int32 InSortOrder)
			: Attribute(InAttribute)
			, Color(InColor)
			, Blend(InBlend)
			, SortOrder(InSortOrder)
		{}

		FName Attribute;
		FLinearColor Color;
		EAnimGraphAttributeBlend Blend;
		int32 SortOrder;
	};

	// Get the attribute info used to draw connections. This varies based on LOD level.
	TArrayView<const FAttributeInfo> GetAttributeInfo() const;

	// Exposes the parent panel's zoom scalar for use when drawing links
	float GetZoomAmount() const;

private:
	void ReconfigureWidgetForAttributes();

	// Get tooltip text with attributes included
	FText GetAttributeTooltipText() const;

protected:
	//~ Begin SGraphPin Interface
	virtual const FSlateBrush* GetPinIcon() const override;
	//~ End SGraphPin Interface

	mutable const FSlateBrush* CachedImg_Pin_ConnectedHovered;
	mutable const FSlateBrush* CachedImg_Pin_DisconnectedHovered;

	TArray<FAttributeInfo> AttributeInfos;
};
