// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationStateNodes/SGraphNodeAnimState.h"

#include "AnimStateConduitNode.h"
#include "AnimStateAliasNode.h"
#include "AnimStateNodeBase.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimInstance.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraph/EdGraphPin.h"
#include "Engine/Blueprint.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/PlatformCrt.h"
#include "IDocumentation.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector2D.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SGraphPanel.h"
#include "SGraphPin.h"
#include "SGraphPreviewer.h"
#include "SNodePanel.h"
#include "SlotBase.h"
#include "SStateMachineInputPin.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "Widgets/Text/STextBlock.h"
#include "SStateMachineOutputPin.h"

class SWidget;
class UEdGraphSchema;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "SGraphNodeAnimState"

/////////////////////////////////////////////////////
// SGraphNodeAnimState

void SGraphNodeAnimState::Construct(const FArguments& InArgs, UAnimStateNodeBase* InNode)
{
	this->GraphNode = InNode;

	this->SetCursor(EMouseCursor::CardinalCross);

	this->UpdateGraphNode();
}

void SGraphNodeAnimState::GetStateInfoPopup(TArray<FGraphInformationPopupInfo>& Popups) const
{
	UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(GraphNode));
	if(AnimBlueprint)
	{
		UAnimInstance* ActiveObject = Cast<UAnimInstance>(AnimBlueprint->GetObjectBeingDebugged());
		UAnimBlueprintGeneratedClass* Class = AnimBlueprint->GetAnimBlueprintGeneratedClass();

		FLinearColor CurrentStateColor = GetBorderBackgroundColor().GetSpecifiedColor();

		// Display various types of debug data
		if ((ActiveObject != NULL) && (Class != NULL))
		{
			if (Class->GetAnimNodeProperties().Num())
			{
				if (FStateMachineDebugData* DebugInfo = Class->GetAnimBlueprintDebugData().StateMachineDebugData.Find(GraphNode->GetGraph()))
				{
					if(int32* StateIndexPtr = DebugInfo->NodeToStateIndex.Find(GraphNode))
					{
						for(const FStateMachineStateDebugData& StateData : Class->GetAnimBlueprintDebugData().StateData)
						{
							if(StateData.StateMachineIndex == DebugInfo->MachineIndex && StateData.StateIndex == *StateIndexPtr)
							{
								if (StateData.Weight > 0.0f)
								{
									FText StateText;
									if (StateData.ElapsedTime > 0.0f)
									{
										StateText = FText::Format(LOCTEXT("ActiveStateWeightFormat", "{0}\nActive for {1}s"), FText::AsPercent(StateData.Weight), FText::AsNumber(StateData.ElapsedTime));
									}
									else
									{
										StateText = FText::Format(LOCTEXT("StateWeightFormat", "{0}"), FText::AsPercent(StateData.Weight));
									}

									Popups.Emplace(nullptr, CurrentStateColor, StateText.ToString());
									break;
								}
							}
						}
					}
				}
			}
		}
	}
}

void SGraphNodeAnimState::GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const
{
	GetStateInfoPopup(Popups);
}

FSlateColor SGraphNodeAnimState::GetBorderBackgroundColor() const
{
	FLinearColor InactiveStateColor(0.08f, 0.08f, 0.08f);
	FLinearColor ActiveStateColorDim = InactiveStateColor;
	FLinearColor ActiveStateColorBright = FStyleColors::AccentOrange.GetSpecifiedColor().Desaturate(0.25f);
	ActiveStateColorBright.A = 1.0f;

	return GetBorderBackgroundColor_Internal(InactiveStateColor, ActiveStateColorDim, ActiveStateColorBright);
}

FSlateColor SGraphNodeAnimState::GetBorderBackgroundColor_Internal(FLinearColor InactiveStateColor, FLinearColor ActiveStateColorDim, FLinearColor ActiveStateColorBright) const
{
	UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(GraphNode));
	if(AnimBlueprint)
	{
		UAnimInstance* ActiveObject = Cast<UAnimInstance>(AnimBlueprint->GetObjectBeingDebugged());
		UAnimBlueprintGeneratedClass* Class = AnimBlueprint->GetAnimBlueprintGeneratedClass();

		// Display various types of debug data
		if ((ActiveObject != NULL) && (Class != NULL))
		{
			if (FStateMachineDebugData* DebugInfo = Class->GetAnimBlueprintDebugData().StateMachineDebugData.Find(GraphNode->GetGraph()))
			{
				if(int32* StateIndexPtr = DebugInfo->NodeToStateIndex.Find(GraphNode))
				{
					for(const FStateMachineStateDebugData& StateData : Class->GetAnimBlueprintDebugData().StateData)
					{
						if(StateData.StateMachineIndex == DebugInfo->MachineIndex && StateData.StateIndex == *StateIndexPtr)
						{
							if (StateData.Weight > 0.0f)
							{
								return FMath::Lerp<FLinearColor>(ActiveStateColorDim, ActiveStateColorBright, StateData.Weight);
							}
						}
					}
				}
			}
		}
	}

	return InactiveStateColor;
}

void SGraphNodeAnimState::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	// Add pins to the hover set so outgoing transitions arrows remains highlighted while the mouse is over the state node
	if (const UAnimStateNodeBase* StateNode = Cast<UAnimStateNodeBase>(GraphNode))
	{
		if (const UEdGraphPin* OutputPin = StateNode->GetOutputPin())
		{
			TSharedPtr<SGraphPanel> OwnerPanel = GetOwnerPanel();
			check(OwnerPanel.IsValid());

			for (int32 LinkIndex = 0; LinkIndex < OutputPin->LinkedTo.Num(); ++LinkIndex)
			{
				OwnerPanel->AddPinToHoverSet(OutputPin->LinkedTo[LinkIndex]);
			}
		}
	}
	
	SGraphNode::OnMouseEnter(MyGeometry, MouseEvent);
}

void SGraphNodeAnimState::OnMouseLeave(const FPointerEvent& MouseEvent)
{
	// Remove manually added pins from the hover set
	if (const UAnimStateNodeBase* StateNode = Cast<UAnimStateNodeBase>(GraphNode))
	{
		if(const UEdGraphPin* OutputPin = StateNode->GetOutputPin())
		{
			TSharedPtr<SGraphPanel> OwnerPanel = GetOwnerPanel();
			check(OwnerPanel.IsValid());

			for (int32 LinkIndex = 0; LinkIndex < OutputPin->LinkedTo.Num(); ++LinkIndex)
			{
				OwnerPanel->RemovePinFromHoverSet(OutputPin->LinkedTo[LinkIndex]);
			}
		}
	}

	SGraphNode::OnMouseLeave(MouseEvent);
}

void SGraphNodeAnimState::UpdateGraphNode()
{
	InputPins.Empty();
	OutputPins.Empty();
	
	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	const FSlateBrush* NodeTypeIcon = GetNameIcon();
	const FSlateBrush* BodyBrush = FAppStyle::GetBrush( "Graph.AnimStateNode.Body" );
	const FSlateBrush* SpillBrush = FAppStyle::GetBrush( "Graph.AnimStateNode.ColorSpill" );
	const FSlateBrush* SelectionBrush = FAppStyle::GetBrush( "Graph.AnimStateNode.Selection" );

	if (GraphNode->IsA<UAnimStateAliasNode>())
	{
		BodyBrush = FAppStyle::GetBrush( "Graph.AnimAliasNode.Body" );
		SpillBrush = FAppStyle::GetBrush( "Graph.AnimAliasNode.ColorSpill" );
		SelectionBrush = FAppStyle::GetBrush( "Graph.AnimAliasNode.Selection" );
	}
	else if (GraphNode->IsA<UAnimStateConduitNode>())
	{
		BodyBrush = FAppStyle::GetBrush( "Graph.AnimConduitNode.Body" );
		SpillBrush = FAppStyle::GetBrush( "Graph.AnimConduitNode.ColorSpill" );
		SelectionBrush = FAppStyle::GetBrush( "Graph.AnimConduitNode.Selection" );
	}

	FLinearColor TitleShadowColor(0.6f, 0.6f, 0.6f);
	TSharedPtr<SErrorText> ErrorText;
	TSharedPtr<SNodeTitle> NodeTitle = SNew(SNodeTitle, GraphNode);

	// Set ourselves as SelfHitTestInvisible so we dont end up with a 1-unit border where the node is selected rather than the pin
	SetVisibility(EVisibility::SelfHitTestInvisible);
	
	this->GetOrAddSlot( ENodeZone::Center )
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SOverlay)
			.Visibility(EVisibility::SelfHitTestInvisible)
			+SOverlay::Slot()
			.Padding(2.0f)
			[
				SNew(SBorder)
				.BorderImage( BodyBrush )
				.Padding(0)
				.BorderBackgroundColor( this, &SGraphNodeAnimState::GetBorderBackgroundColor )
				[
					SNew(SOverlay)

					// PIN AREA
					+SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SAssignNew(RightNodeBox, SVerticalBox)
						+SVerticalBox::Slot()
						.HAlign(HAlign_Fill)
						.VAlign(VAlign_Fill)
						.FillHeight(1.0f)
						[
							SAssignNew(PinOverlay, SOverlay)
						]
					]

					// STATE NAME AREA
					+SOverlay::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(12.0f)
					[
						SNew(SBorder)
						.BorderImage( SpillBrush )
						.BorderBackgroundColor( TitleShadowColor )
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.Padding(FMargin(2.0f, 2.0f, 0.0f, 2.0f))
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							[
								// POPUP ERROR MESSAGE
								SAssignNew(ErrorText, SErrorText )
								.BackgroundColor( this, &SGraphNodeAnimState::GetErrorColor )
								.ToolTipText( this, &SGraphNodeAnimState::GetErrorMsgToolTip )
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(FMargin(7.0f, 4.0f, 2.0f, 4.0f))
							[
								SNew(SImage)
								.Image(NodeTypeIcon)
								.ColorAndOpacity( this, &SGraphNodeAnimState::GetIconColor )
							]
							+SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Center)
							.AutoWidth()
							[
								SNew(SVerticalBox)
								+SVerticalBox::Slot()
								.AutoHeight()
								.Padding(MakeAttributeLambda([this]() -> FMargin
								{
									return InlineEditableText->IsInEditMode() ?
										FMargin(0.0f, 0.0f, 0.0f, 0.0f) :
										FMargin(5.0f, 0.0f, 13.0f, 0.0f);
								}))
								[
									SAssignNew(InlineEditableText, SInlineEditableTextBlock)
									.Style( FAppStyle::Get(), "Graph.AnimStateNode.NodeTitleInlineEditableText" )
									.Text( NodeTitle.Get(), &SNodeTitle::GetHeadTitle )
									.OnVerifyTextChanged(this, &SGraphNodeAnimState::OnVerifyNameTextChanged)
									.OnTextCommitted(this, &SGraphNodeAnimState::OnNameTextCommited)
									.IsReadOnly( this, &SGraphNodeAnimState::IsNameReadOnly )
									.IsSelected(this, &SGraphNodeAnimState::IsSelectedExclusively)
								]
								+SVerticalBox::Slot()
								.AutoHeight()
								[
									NodeTitle.ToSharedRef()
								]
							]
						]
					]
				]
			]
			+SOverlay::Slot()
			[
				SNew(SBorder)
				.BorderImage(SelectionBrush)
				.Padding(0)
				.Visibility_Lambda([this]()
				{
					TSharedPtr<SGraphPanel> OwnerPanel = OwnerGraphPanelPtr.Pin();
					if (!OwnerPanel.IsValid())
					{
						return EVisibility::Hidden;
					}

					return OwnerPanel->SelectionManager.IsNodeSelected(GraphNode) ? EVisibility::HitTestInvisible : EVisibility::Hidden;
				})
			]
		];

	ErrorReporting = ErrorText;
	ErrorReporting->SetError(ErrorMsg);
	CreatePinWidgets();
}

void SGraphNodeAnimState::CreatePinWidgets()
{
	UAnimStateNodeBase* StateNode = CastChecked<UAnimStateNodeBase>(GraphNode);

	UEdGraphPin* OutputPin = StateNode->GetOutputPin();
	if (!OutputPin->bHidden)
	{
		TSharedPtr<SGraphPin> NewPin = SNew(SStateMachineOutputPin, OutputPin);
		this->AddPin(NewPin.ToSharedRef());
	}
	UEdGraphPin* InputPin = StateNode->GetInputPin();
	if (!InputPin->bHidden)
	{
		TSharedPtr<SGraphPin> NewPin = SNew(SStateMachineInputPin, InputPin);
		this->AddPin(NewPin.ToSharedRef());
	}
}

void SGraphNodeAnimState::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{
	PinToAdd->SetOwner(SharedThis(this));
	PinOverlay->AddSlot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		[
			PinToAdd
		];

	switch (PinToAdd->GetPinObj()->Direction)
	{
	case EGPD_Input:
		InputPins.Add(PinToAdd);
		break;
	case EGPD_Output:
		OutputPins.Add(PinToAdd);
		break;
	default:
		break;
	}
}

TSharedPtr<SToolTip> SGraphNodeAnimState::GetComplexTooltip()
{
	UAnimStateNodeBase* StateNode = CastChecked<UAnimStateNodeBase>(GraphNode);

	return SNew(SToolTip)
		[
			SNew(SVerticalBox)
	
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				// Create the tooltip preview, ensure to disable state overlays to stop
				// PIE and read-only borders obscuring the graph
				SNew(SGraphPreviewer, StateNode->GetBoundGraph())
				.CornerOverlayText(this, &SGraphNodeAnimState::GetPreviewCornerText)
				.ShowGraphStateOverlay(false)
			]
	
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(FMargin(0.0f, 5.0f, 0.0f, 0.0f))
			[
				IDocumentation::Get()->CreateToolTip(FText::FromString("Documentation"), NULL, StateNode->GetDocumentationLink(), StateNode->GetDocumentationExcerptName())
			]

		];
}

FText SGraphNodeAnimState::GetPreviewCornerText() const
{
	UAnimStateNodeBase* StateNode = CastChecked<UAnimStateNodeBase>(GraphNode);

	return FText::Format(NSLOCTEXT("SGraphNodeAnimState", "PreviewCornerStateText", "{0} state"), FText::FromString(StateNode->GetStateName()));
}

const FSlateBrush* SGraphNodeAnimState::GetNameIcon() const
{
	return FAppStyle::GetBrush( TEXT("Graph.StateNode.Icon") );
}

FSlateColor SGraphNodeAnimState::GetIconColor() const
{
	return FLinearColor(0.8f, 0.8f, 0.8f);
}

const FSlateBrush* SGraphNodeAnimState::GetShadowBrush(bool bSelected) const
{
	if (GraphNode->IsA<UAnimStateAliasNode>())
	{
		return FAppStyle::GetBrush(TEXT("Graph.AnimAliasNode.Shadow"));
	}

	return FAppStyle::GetBrush(TEXT("Graph.Node.Shadow"));
}

/////////////////////////////////////////////////////
// SGraphNodeAnimConduit

void SGraphNodeAnimConduit::Construct(const FArguments& InArgs, UAnimStateConduitNode* InNode)
{
	SGraphNodeAnimState::Construct(SGraphNodeAnimState::FArguments(), InNode);
}

void SGraphNodeAnimConduit::GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const
{
	// Intentionally empty.
}

FSlateColor SGraphNodeAnimConduit::GetBorderBackgroundColor_Internal(FLinearColor InactiveStateColor, FLinearColor ActiveStateColorDim, FLinearColor ActiveStateColorBright) const
{
	// Override inactive state color for conduits.
	return SGraphNodeAnimState::GetBorderBackgroundColor_Internal(FLinearColor(0.38f, 0.45f, 0.21f), ActiveStateColorDim, ActiveStateColorBright);
}

FText SGraphNodeAnimConduit::GetPreviewCornerText() const
{
	UAnimStateNodeBase* StateNode = CastChecked<UAnimStateNodeBase>(GraphNode);

	return FText::Format(NSLOCTEXT("SGraphNodeAnimState", "PreviewCornerConduitText", "{0} conduit"), FText::FromString(StateNode->GetStateName()));
}

const FSlateBrush* SGraphNodeAnimConduit::GetNameIcon() const
{
	return FAppStyle::GetBrush( TEXT("Graph.ConduitNode.Icon") );
}

FSlateColor SGraphNodeAnimConduit::GetIconColor() const
{
	return FLinearColor(0.38f, 0.45f, 0.21f);
}

#undef LOCTEXT_NAMESPACE