// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "SGraphNode.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SGraphPin;
class SToolTip;
class UAnimStateConduitNode;
class UAnimStateNodeBase;
class UEdGraphNode;
struct FGeometry;
struct FGraphInformationPopupInfo;
struct FNodeInfoContext;
struct FPointerEvent;
struct FSlateBrush;

class SGraphNodeAnimState : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeAnimState){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UAnimStateNodeBase* InNode);

	// SNodePanel::SNode interface
	virtual void GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const override;
	// End of SNodePanel::SNode interface

	// SGraphNode interface
	virtual void UpdateGraphNode() override;
	virtual void CreatePinWidgets() override;
	virtual void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;
	virtual TSharedPtr<SToolTip> GetComplexTooltip() override;
	virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	// End of SGraphNode interface

	// SWidget interface
	void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	// End of SWidget interface

	void GetStateInfoPopup(TArray<FGraphInformationPopupInfo>& Popups) const;
protected:
	FSlateColor GetBorderBackgroundColor() const;
	virtual FSlateColor GetBorderBackgroundColor_Internal(FLinearColor InactiveStateColor, FLinearColor ActiveStateColorDim, FLinearColor ActiveStateColorBright) const;
	virtual FText GetPreviewCornerText() const;
	virtual const FSlateBrush* GetNameIcon() const;
	virtual FSlateColor GetIconColor() const;

	TSharedPtr<SOverlay> PinOverlay;
};


class SGraphNodeAnimConduit : public SGraphNodeAnimState
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeAnimConduit){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UAnimStateConduitNode* InNode);

	// SNodePanel::SNode interface
	virtual void GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const override;
	// End of SNodePanel::SNode interface
protected:
	virtual FSlateColor GetBorderBackgroundColor_Internal(FLinearColor InactiveStateColor, FLinearColor ActiveStateColorDim, FLinearColor ActiveStateColorBright) const override;
	virtual FText GetPreviewCornerText() const override;
	virtual const FSlateBrush* GetNameIcon() const override;
	virtual FSlateColor GetIconColor() const override;
};
