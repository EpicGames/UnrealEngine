// Copyright Epic Games, Inc. All Rights Reserved.

#include "AnimationStateNodes/SGraphNodeAnimStateAlias.h"

#include "AnimStateAliasNode.h"
#include "AnimStateNodeBase.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimInstance.h"
#include "AnimationStateMachineGraph.h"
#include "AnimationStateNodes/SGraphNodeAnimState.h"
#include "AnimationStateNodes/SGraphNodeAnimTransition.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "EdGraph/EdGraphNode.h"
#include "Engine/Blueprint.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Math/UnrealMathSSE.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"

class SToolTip;
struct FGraphInformationPopupInfo;
struct FNodeInfoContext;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "SGraphNodeAnimStateAlias"

/////////////////////////////////////////////////////
// SGraphNodeAnimStateAlias

void SGraphNodeAnimStateAlias::Construct(const FArguments& InArgs, UAnimStateAliasNode* InNode)
{
	SGraphNodeAnimState::Construct(SGraphNodeAnimState::FArguments(), InNode);
}

void SGraphNodeAnimStateAlias::GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const
{
	GetStateInfoPopup(GraphNode, Popups);
}

FSlateColor SGraphNodeAnimStateAlias::GetBorderBackgroundColor_Internal(FLinearColor InactiveStateColor, FLinearColor ActiveStateColorDim, FLinearColor ActiveStateColorBright) const
{
	UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(GraphNode));
	if (AnimBlueprint)
	{
		UAnimInstance* ActiveObject = Cast<UAnimInstance>(AnimBlueprint->GetObjectBeingDebugged());
		UAnimBlueprintGeneratedClass* Class = AnimBlueprint->GetAnimBlueprintGeneratedClass();

		// Display various types of debug data
		if ((ActiveObject != NULL) && (Class != NULL))
		{
			UAnimationStateMachineGraph* StateMachineGraph = CastChecked<UAnimationStateMachineGraph>(GraphNode->GetGraph());
			UAnimStateAliasNode* StateAliasNode = CastChecked<UAnimStateAliasNode>(GraphNode);

			if (FStateMachineDebugData* DebugInfo = Class->GetAnimBlueprintDebugData().StateMachineDebugData.Find(StateMachineGraph))
			{
				TArray<int32, TInlineAllocator<16>> StatesToCheck;

				// Check transitions associated with our state alias.
				TArray<FStateMachineDebugData::FStateAliasTransitionStateIndexPair> TransitionStatePairs;
				DebugInfo->StateAliasNodeToTransitionStatePairs.MultiFind(StateAliasNode, TransitionStatePairs);
				const int32 PairsNum = TransitionStatePairs.Num();
				for (int32 Index = 0; Index < PairsNum; ++Index)
				{
					auto& TransStatePair = TransitionStatePairs[Index];
					if (SGraphNodeAnimTransition::IsTransitionActive(TransStatePair.TransitionIndex, *Class, *StateMachineGraph, *ActiveObject))
					{
						// Add the prev/next state we are aliasing that's associated with this transition.
						StatesToCheck.Add(TransStatePair.AssociatedStateIndex);
					}
				}

				// Use the highest aliased state's weight that has an active transition in/out of this alias. 
				float Weight = 0.0f;
				for (const int32 StateIndex : StatesToCheck)
				{
					if (TWeakObjectPtr<UAnimStateNodeBase>* StateNodeWeakPtr = DebugInfo->StateIndexToNode.Find(StateIndex))
					{
						if (StateAliasNode->GetAliasedStates().Contains(*StateNodeWeakPtr))
						{
							for (const FStateMachineStateDebugData& StateData : Class->GetAnimBlueprintDebugData().StateData)
							{
								if (StateData.StateMachineIndex == DebugInfo->MachineIndex && StateData.StateIndex == StateIndex)
								{
									Weight = FMath::Max(Weight, StateData.Weight);
								}
							}
						}
					}
				}

				if (Weight > 0.0f)
				{
					return FMath::Lerp<FLinearColor>(ActiveStateColorDim, ActiveStateColorBright, Weight);
				}
			}
		}
	}

	// Override inactive state color for aliases.
	return FLinearColor(0.45f, 0.33f, 0.37f);
}

FText SGraphNodeAnimStateAlias::GetPreviewCornerText() const
{
	return FText();
}

const FSlateBrush* SGraphNodeAnimStateAlias::GetNameIcon() const
{
	return FAppStyle::GetBrush(TEXT("Graph.AliasNode.Icon"));
}

FSlateColor SGraphNodeAnimStateAlias::GetIconColor() const
{
	return FLinearColor(0.45f, 0.33f, 0.37f);
}

TSharedPtr<SToolTip> SGraphNodeAnimStateAlias::GetComplexTooltip()
{
	return nullptr;
}

void SGraphNodeAnimStateAlias::GetStateInfoPopup(UEdGraphNode* GraphNode, TArray<FGraphInformationPopupInfo>& Popups)
{
	UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNode(GraphNode));
	if (AnimBlueprint)
	{
		UAnimInstance* ActiveObject = Cast<UAnimInstance>(AnimBlueprint->GetObjectBeingDebugged());
		UAnimBlueprintGeneratedClass* Class = AnimBlueprint->GetAnimBlueprintGeneratedClass();

		static const FLinearColor PopupColor(1.f, 0.5f, 0.25f);

		// Display various types of debug data
		if ((ActiveObject != NULL) && (Class != NULL))
		{
			UAnimationStateMachineGraph* StateMachineGraph = CastChecked<UAnimationStateMachineGraph>(GraphNode->GetGraph());
			UAnimStateAliasNode* StateAliasNode = CastChecked<UAnimStateAliasNode>(GraphNode);

			if (FStateMachineDebugData* DebugInfo = Class->GetAnimBlueprintDebugData().StateMachineDebugData.Find(StateMachineGraph))
			{
				TArray<int32, TInlineAllocator<16>> StatesToCheck;

				// Check transitions associated with our state alias.
				TArray<FStateMachineDebugData::FStateAliasTransitionStateIndexPair> TransitionStatePairs;
				DebugInfo->StateAliasNodeToTransitionStatePairs.MultiFind(StateAliasNode, TransitionStatePairs);
				const int32 PairsNum = TransitionStatePairs.Num();
				for (int32 Index = 0; Index < PairsNum; ++Index)
				{
					auto& TransStatePair = TransitionStatePairs[Index];
					if (SGraphNodeAnimTransition::IsTransitionActive(TransStatePair.TransitionIndex, *Class, *StateMachineGraph, *ActiveObject))
					{
						// Add the prev/next state we are aliasing that's associated with this transition.
						StatesToCheck.Add(TransStatePair.AssociatedStateIndex);
					}
				}

				// Display the name and weight of any state we are aliasing that has an active incoming/outgoing transition from this alias
				for (const int32 StateIndex : StatesToCheck)
				{
					if (TWeakObjectPtr<UAnimStateNodeBase>* StateNodeWeakPtr = DebugInfo->StateIndexToNode.Find(StateIndex))
					{
						if (UAnimStateNodeBase* StateNode = StateNodeWeakPtr->Get())
						{
							if (StateAliasNode->bGlobalAlias || StateAliasNode->GetAliasedStates().Contains(StateNode))
							{
								for (const FStateMachineStateDebugData& StateData : Class->GetAnimBlueprintDebugData().StateData)
								{
									if (StateData.StateMachineIndex == DebugInfo->MachineIndex && StateData.StateIndex == StateIndex)
									{
										FText StateText;
										StateText = FText::Format(LOCTEXT("ActiveAliasedStateFormat", "{0}(Alias) {1}\nActive for {2}s"),
											FText::FromString(StateNode->GetStateName()),
											FText::AsPercent(StateData.Weight),
											FText::AsNumber(StateData.ElapsedTime));
										Popups.Emplace(nullptr, PopupColor, StateText.ToString());
										break;
									}
								}
							}
						}
					}
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE