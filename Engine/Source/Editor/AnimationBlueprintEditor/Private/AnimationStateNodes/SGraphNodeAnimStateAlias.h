// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "SGraphNodeAnimState.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SToolTip;
class UAnimStateAliasNode;
class UEdGraphNode;
struct FGraphInformationPopupInfo;
struct FNodeInfoContext;
struct FSlateBrush;

class SGraphNodeAnimStateAlias : public SGraphNodeAnimState
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeAnimStateAlias) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UAnimStateAliasNode* InNode);

	// SNodePanel::SNode interface
	virtual void GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const override;
	virtual TSharedPtr<SToolTip> GetComplexTooltip() override;
	// End of SNodePanel::SNode interface

	static void GetStateInfoPopup(UEdGraphNode* GraphNode, TArray<FGraphInformationPopupInfo>& Popups);

protected:
	virtual FSlateColor GetBorderBackgroundColor_Internal(FLinearColor InactiveStateColor, FLinearColor ActiveStateColorDim, FLinearColor ActiveStateColorBright) const;
	virtual FText GetPreviewCornerText() const override;
	virtual const FSlateBrush* GetNameIcon() const override;
	virtual FSlateColor GetIconColor() const override;
};
