// Copyright Epic Games, Inc. All Rights Reserved.


#include "AnimationStateNodes/SGraphNodeAnimStateEntry.h"

#include "AnimStateEntryNode.h"
#include "SGraphPanel.h"
#include "Delegates/Delegate.h"
#include "GenericPlatform/ICursor.h"
#include "HAL/PlatformCrt.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Margin.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "Misc/Optional.h"
#include "SGraphPin.h"
#include "SNodePanel.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Types/SlateEnums.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "SStateMachineOutputPin.h"
#include "Widgets/Images/SImage.h"

#define LOCTEXT_NAMESPACE "SGraphNodeAnimStateEntry"

/////////////////////////////////////////////////////
// SGraphNodeAnimStateEntry

void SGraphNodeAnimStateEntry::Construct(const FArguments& InArgs, UAnimStateEntryNode* InNode)
{
	this->GraphNode = InNode;

	this->SetCursor(EMouseCursor::CardinalCross);

	this->UpdateGraphNode();
}

void SGraphNodeAnimStateEntry::GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const
{

}

FSlateColor SGraphNodeAnimStateEntry::GetBorderBackgroundColor() const
{
	FLinearColor InactiveStateColor(0.08f, 0.08f, 0.08f);
	FLinearColor ActiveStateColorDim(0.4f, 0.3f, 0.15f);
	FLinearColor ActiveStateColorBright(1.f, 0.6f, 0.35f);

	return InactiveStateColor;
}

void SGraphNodeAnimStateEntry::UpdateGraphNode()
{
	InputPins.Empty();
	OutputPins.Empty();
	
	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();


	// Set ourselves as SelfHitTestInvisible so we dont end up with a 1-unit border where the node is selected rather than the pin
	SetVisibility(EVisibility::SelfHitTestInvisible);

	FLinearColor TitleShadowColor(0.6f, 0.6f, 0.6f);

	this->GetOrAddSlot( ENodeZone::Center )
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SOverlay)
			.Visibility(EVisibility::SelfHitTestInvisible)
			+SOverlay::Slot()
			.Padding(1.0f)
			[
				SNew(SBorder)
				.BorderImage( FAppStyle::GetBrush( "Graph.AnimConduitNode.Body" ) )
				.Padding(0)
				.BorderBackgroundColor( this, &SGraphNodeAnimStateEntry::GetBorderBackgroundColor )
				[
					SNew(SOverlay)

					// PIN AREA
					+SOverlay::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SAssignNew(RightNodeBox, SVerticalBox)
					]
					// STATE NAME AREA
					+SOverlay::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.Padding(12.0f)
					[
						SNew(SBorder)
						.BorderImage( FAppStyle::GetBrush( "Graph.AnimConduitNode.ColorSpill" ) )
						.BorderBackgroundColor( TitleShadowColor )
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.Padding(FMargin(2.0f, 2.0f, 2.0f, 2.0f))
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(FMargin(8.0f, 0.0f, 4.0f, 0.0f))
							[
								SNew(SImage)
								.Image(FAppStyle::GetBrush("Graph.EntryNode.Icon"))
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.VAlign(VAlign_Center)
							.Padding(FMargin(2.0f, 2.5f, 8.0f, 2.5f))
							[
								SNew(STextBlock)
								.Text(LOCTEXT("EntryName", "Entry"))
								.TextStyle(FAppStyle::Get(), "Graph.AnimStateNode.NodeTitle")
							]
						]
					]
				]
			]
			+SOverlay::Slot()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush( "Graph.AnimConduitNode.Selection"))
				.Padding(0)
				.Visibility_Lambda([this]()
				{
					TSharedPtr<SGraphPanel> OwnerPanel = OwnerGraphPanelPtr.Pin();
					if (!OwnerPanel.IsValid())
					{
						return EVisibility::Hidden;
					}

					return OwnerPanel->SelectionManager.IsNodeSelected(GraphNode) ? EVisibility::HitTestInvisible : EVisibility::Hidden;
				})
			]
		];

	CreatePinWidgets();
}

void SGraphNodeAnimStateEntry::CreatePinWidgets()
{
	UAnimStateEntryNode* StateNode = CastChecked<UAnimStateEntryNode>(GraphNode);

	UEdGraphPin* CurPin = StateNode->GetOutputPin();
	if (!CurPin->bHidden)
	{
		TSharedPtr<SGraphPin> NewPin = SNew(SStateMachineOutputPin, CurPin);

		this->AddPin(NewPin.ToSharedRef());
	}
}

void SGraphNodeAnimStateEntry::AddPin(const TSharedRef<SGraphPin>& PinToAdd)
{
	PinToAdd->SetOwner( SharedThis(this) );
	RightNodeBox->AddSlot()
		.HAlign(HAlign_Fill)
		.VAlign(VAlign_Fill)
		.FillHeight(1.0f)
		[
			PinToAdd
		];
	OutputPins.Add(PinToAdd);
}

FText SGraphNodeAnimStateEntry::GetPreviewCornerText() const
{
	return LOCTEXT("CornerTextDescription", "Entry point for state machine");
}

const FSlateBrush* SGraphNodeAnimStateEntry::GetShadowBrush(bool bSelected) const
{
	return FAppStyle::GetBrush(TEXT("Graph.Node.Shadow"));
}

#undef LOCTEXT_NAMESPACE