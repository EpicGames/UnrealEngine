// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Internationalization/Text.h"
#include "SGraphNode.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class SGraphPin;
class UAnimStateEntryNode;
struct FGraphInformationPopupInfo;
struct FNodeInfoContext;

class SGraphNodeAnimStateEntry : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeAnimStateEntry){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UAnimStateEntryNode* InNode);

	// SNodePanel::SNode interface
	virtual void GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const override;
	// End of SNodePanel::SNode interface

	// SGraphNode interface
	virtual void UpdateGraphNode() override;
	virtual void AddPin(const TSharedRef<SGraphPin>& PinToAdd) override;
	virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	virtual void CreatePinWidgets() override;
	// End of SGraphNode interface

	
protected:
	FSlateColor GetBorderBackgroundColor() const;

	FText GetPreviewCornerText() const;
};
