// Copyright Epic Games, Inc. All Rights Reserved.


#include "AnimationStateNodes/SGraphNodeAnimTransition.h"

#include "AnimGraphNode_Base.h"
#include "AnimGraphNode_StateMachineBase.h"
#include "AnimGraphNode_TransitionResult.h"
#include "AnimStateNodeBase.h"
#include "AnimStateTransitionNode.h"
#include "Animation/AnimBlueprint.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimNode_StateMachine.h"
#include "Animation/AnimStateMachineTypes.h"
#include "AnimationStateMachineGraph.h"
#include "AnimationTransitionGraph.h"
#include "ConnectionDrawingPolicy.h"
#include "Containers/EnumAsByte.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraphNode.h"
#include "EdGraphSchema_K2.h"
#include "Engine/Blueprint.h"
#include "HAL/PlatformCrt.h"
#include "IDocumentation.h"
#include "Internationalization/Internationalization.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/Geometry.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Attribute.h"
#include "SGraphPanel.h"
#include "SKismetLinearExpression.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SOverlay.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

class SWidget;
class UEdGraphPin;
class UObject;
struct FPointerEvent;
struct FSlateBrush;

#define LOCTEXT_NAMESPACE "TransitionNodes"

/////////////////////////////////////////////////////
// SGraphNodeAnimTransition

void SGraphNodeAnimTransition::Construct(const FArguments& InArgs, UAnimStateTransitionNode* InNode)
{
	this->GraphNode = InNode;
	this->UpdateGraphNode();
}

void SGraphNodeAnimTransition::GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const
{
}

void SGraphNodeAnimTransition::MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty)
{
	// Ignored; position is set by the location of the attached state nodes
}

bool SGraphNodeAnimTransition::RequiresSecondPassLayout() const
{
	return true;
}

void SGraphNodeAnimTransition::PerformSecondPassLayout(const TMap< UObject*, TSharedRef<SNode> >& NodeToWidgetLookup) const
{
	UAnimStateTransitionNode* TransNode = CastChecked<UAnimStateTransitionNode>(GraphNode);

	// Find the geometry of the state nodes we're connecting
	FGeometry StartGeom;
	FGeometry EndGeom;

	int32 TransIndex = 0;
	int32 NumOfTrans = 1;

	UAnimStateNodeBase* PrevState = TransNode->GetPreviousState();
	UAnimStateNodeBase* NextState = TransNode->GetNextState();
	if ((PrevState != NULL) && (NextState != NULL))
	{
		const TSharedRef<SNode>* pPrevNodeWidget = NodeToWidgetLookup.Find(PrevState);
		const TSharedRef<SNode>* pNextNodeWidget = NodeToWidgetLookup.Find(NextState);
		if ((pPrevNodeWidget != NULL) && (pNextNodeWidget != NULL))
		{
			const TSharedRef<SNode>& PrevNodeWidget = *pPrevNodeWidget;
			const TSharedRef<SNode>& NextNodeWidget = *pNextNodeWidget;

			StartGeom = FGeometry(FVector2D(PrevState->NodePosX, PrevState->NodePosY), FVector2D::ZeroVector, PrevNodeWidget->GetDesiredSize(), 1.0f);
			EndGeom = FGeometry(FVector2D(NextState->NodePosX, NextState->NodePosY), FVector2D::ZeroVector, NextNodeWidget->GetDesiredSize(), 1.0f);

			TArray<UAnimStateTransitionNode*> Transitions;
			PrevState->GetTransitionList(Transitions);

			Transitions = Transitions.FilterByPredicate([NextState](const UAnimStateTransitionNode* InTransition) -> bool
			{
				return InTransition->GetNextState() == NextState;
			});

			TransIndex = Transitions.IndexOfByKey(TransNode);
			NumOfTrans = Transitions.Num();

			PrevStateNodeWidgetPtr = PrevNodeWidget;
		}
	}

	//Position Node
	PositionBetweenTwoNodesWithOffset(StartGeom, EndGeom, TransIndex, NumOfTrans);
}

TSharedRef<SWidget> SGraphNodeAnimTransition::GenerateRichTooltip()
{
	UAnimStateTransitionNode* TransNode = CastChecked<UAnimStateTransitionNode>(GraphNode);

	if (TransNode->BoundGraph == NULL)
	{
		return SNew(STextBlock).Text(LOCTEXT("NoAnimGraphBoundToNodeMessage", "Error: No graph"));
	}

	// Find the expression hooked up to the can execute pin of the transition node
	const UEdGraphSchema_K2* Schema = GetDefault<UEdGraphSchema_K2>();

	UEdGraphPin* CanExecPin = NULL;

	if (UAnimationTransitionGraph* TransGraph = Cast<UAnimationTransitionGraph>(TransNode->BoundGraph))
	{
		if (UAnimGraphNode_TransitionResult* ResultNode = TransGraph->GetResultNode())
		{
			CanExecPin = ResultNode->FindPin(TEXT("bCanEnterTransition"));
		}
	}

	TSharedRef<SVerticalBox> Widget = SNew(SVerticalBox);

	const FText TooltipDesc = GetPreviewCornerText(false);

	
	// Transition rule linearized
	Widget->AddSlot()
		.AutoHeight()
		.Padding( 2.0f )
		[
			SNew(STextBlock)
			.TextStyle( FAppStyle::Get(), TEXT("Graph.TransitionNode.TooltipName") )
			.Text(TooltipDesc)
		];

	if(TransNode->bAutomaticRuleBasedOnSequencePlayerInState)
	{
		if (CanExecPin != nullptr && CanExecPin->LinkedTo.Num() > 0)
		{
			Widget->AddSlot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), TEXT("Graph.TransitionNode.TooltipRule"))
					.Text(LOCTEXT("AnimGraphNodeAutomaticRuleWarning_ToolTip", "Warning : Automatic Rule Based Transition will override graph exit rule."))
					.ColorAndOpacity(FCoreStyle::Get().GetColor("ErrorReporting.WarningBackgroundColor"))
				];
		}
		else
		{
			Widget->AddSlot()
				.AutoHeight()
				.Padding(2.0f)
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), TEXT("Graph.TransitionNode.TooltipRule"))
					.Text(LOCTEXT("AnimGraphNodeAutomaticRule_ToolTip", "Automatic Rule"))
				];
		}
	}
	else
	{
		Widget->AddSlot()
		.AutoHeight()
		.Padding( 2.0f )
		[
			SNew(STextBlock)
			.TextStyle( FAppStyle::Get(), TEXT("Graph.TransitionNode.TooltipRule") )
			.Text(LOCTEXT("AnimGraphNodeTransitionRule_ToolTip", "Transition Rule (in words)"))
		];

		Widget->AddSlot()
			.AutoHeight()
			.Padding( 2.0f )
			[
				SNew(SKismetLinearExpression, CanExecPin)
			];
	}

	if (TransNode->bDisabled)
	{
		Widget->AddSlot()
			.AutoHeight()
			.Padding(2.0f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("TransitionRuleDisabled", "Disabled"))
			];
	}

	Widget->AddSlot()
		.AutoHeight()
		.Padding( 2.0f )
		[
			IDocumentation::Get()->CreateToolTip(FText::FromString("Documentation"), NULL, TransNode->GetDocumentationLink(), TransNode->GetDocumentationExcerptName())
		];
			
	return Widget;
}

TSharedPtr<SToolTip> SGraphNodeAnimTransition::GetComplexTooltip()
{
	return SNew(SToolTip)
		[
			GenerateRichTooltip()
		];
}

void SGraphNodeAnimTransition::UpdateGraphNode()
{
	InputPins.Empty();
	OutputPins.Empty();

	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	TSharedRef<SImage> DirectionImage = SNew(SImage)
		.Image(this, &SGraphNodeAnimTransition::GetTransitionIconImage)
		.ColorAndOpacity(FStyleColors::Background);

	DirectionImage->SetRenderTransform(MakeAttributeLambda([this]()-> TOptional<FSlateRenderTransform>
	{
		UAnimStateTransitionNode* TransNode = CastChecked<UAnimStateTransitionNode>(GraphNode);
		return FSlateRenderTransform(FQuat2D(TransNode->CachedRotation));
	}));
	DirectionImage->SetRenderTransformPivot(FVector2D(0.5f, 0.5f));

	this->GetOrAddSlot( ENodeZone::Center )
		.HAlign(HAlign_Center)
		.VAlign(VAlign_Center)
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			.Padding(2.0f)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Graph.AnimTransitionNode.ColorSpill"))
				.ColorAndOpacity(this, &SGraphNodeAnimTransition::GetTransitionColor)
			]
			+SOverlay::Slot()
			[
				SNew(SBox)
				.Padding(4.0f)
				[
					DirectionImage
				]
			]
			+SOverlay::Slot()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("Graph.AnimTransitionNode.Selection"))
				.Padding(0)
				.Visibility_Lambda([this]()
				{
					TSharedPtr<SGraphPanel> OwnerPanel = OwnerGraphPanelPtr.Pin();
					if (!OwnerPanel.IsValid())
					{
						return EVisibility::Hidden;
					}

					return OwnerPanel->SelectionManager.IsNodeSelected(GraphNode) ? EVisibility::HitTestInvisible : EVisibility::Hidden;
				})
			]
		];
}

FText SGraphNodeAnimTransition::GetPreviewCornerText(bool bReverse) const
{
	UAnimStateTransitionNode* TransNode = CastChecked<UAnimStateTransitionNode>(GraphNode);

	UAnimStateNodeBase* PrevState = (bReverse ? TransNode->GetNextState() : TransNode->GetPreviousState());
	UAnimStateNodeBase* NextState = (bReverse ? TransNode->GetPreviousState() : TransNode->GetNextState());

	FText Result = LOCTEXT("BadTransition", "Bad transition (missing source or target)");

	// Show the priority if there is any ambiguity
	if (PrevState != NULL)
	{
		if (NextState != NULL)
		{
			TArray<UAnimStateTransitionNode*> TransitionFromSource;
			PrevState->GetTransitionList(/*out*/ TransitionFromSource);

			bool bMultiplePriorities = false;
			if (TransitionFromSource.Num() > 1)
			{
				// See if the priorities differ
				for (int32 Index = 0; (Index < TransitionFromSource.Num()) && !bMultiplePriorities; ++Index)
				{
					const bool bDifferentPriority = (TransitionFromSource[Index]->PriorityOrder != TransNode->PriorityOrder);
					bMultiplePriorities |= bDifferentPriority;
				}
			}

			if (bMultiplePriorities)
			{
				Result = FText::Format(LOCTEXT("TransitionXToYWithPriority", "{0} to {1} (Priority {2})"), FText::FromString(PrevState->GetStateName()), FText::FromString(NextState->GetStateName()), FText::AsNumber(TransNode->PriorityOrder));
			}
			else
			{
				Result = FText::Format(LOCTEXT("TransitionXToY", "{0} to {1}"), FText::FromString(PrevState->GetStateName()), FText::FromString(NextState->GetStateName()));
			}
		}
	}

	return Result;
}

FSlateColor SGraphNodeAnimTransition::StaticGetTransitionColor(UAnimStateTransitionNode* TransNode, bool bIsHovered)
{
	FLinearColor ActiveColor = FStyleColors::AccentOrange.GetSpecifiedColor().Desaturate(0.25f);
	ActiveColor.A = 1.0f;
	const FSlateColor HoverColor = FStyleColors::AccentOrange;
	FSlateColor BaseColor = FStyleColors::Foreground;

	// Display various types of debug data
	UAnimBlueprint* AnimBlueprint = Cast<UAnimBlueprint>(FBlueprintEditorUtils::FindBlueprintForNodeChecked(TransNode));
	check(AnimBlueprint);
	UAnimInstance* ActiveObject = Cast<UAnimInstance>(AnimBlueprint->GetObjectBeingDebugged());
	UAnimBlueprintGeneratedClass* Class = AnimBlueprint->GetAnimBlueprintGeneratedClass();

	//@TODO: WIP fast path / slow path coloring
	if (AnimBlueprint->bWarnAboutBlueprintUsage || ((ActiveObject != nullptr) && (ActiveObject->PCV_ShouldNotifyAboutNodesNotUsingFastPath() || ActiveObject->PCV_ShouldWarnAboutNodesNotUsingFastPath())))
	{
		if (UAnimationTransitionGraph* TransGraph = Cast<UAnimationTransitionGraph>(TransNode->GetBoundGraph()))
		{
			if (UAnimGraphNode_TransitionResult* ResultNode = TransGraph->GetResultNode())
			{
				if (ResultNode->BlueprintUsage == EBlueprintUsage::UsesBlueprint)
				{
					BaseColor = FLinearColor(0.4f, 0.4f, 1.0f);
				}
			}
		}
	}

	if ((ActiveObject != NULL) && (Class != NULL))
	{
		UAnimationStateMachineGraph* StateMachineGraph = CastChecked<UAnimationStateMachineGraph>(TransNode->GetGraph());
		if (FStateMachineDebugData* DebugInfo = Class->GetAnimBlueprintDebugData().StateMachineDebugData.Find(StateMachineGraph))
		{
			// A transition node could be associated with multiple transitions indicies when coming from an alias. Check all of them
			TArray<int32> TransitionIndices;
			DebugInfo->NodeToTransitionIndex.MultiFind(TransNode, TransitionIndices);
			const int32 TransNum = TransitionIndices.Num();
			for (int32 Index = 0; Index < TransNum; ++Index)
			{
				if(IsTransitionActive(TransitionIndices[Index], *Class, *StateMachineGraph, *ActiveObject))
				{	
					// We're active!
					return ActiveColor;
				}
			}
		}
	}

	//@TODO: ANIMATION: Sort out how to display this
	// 			if (TransNode->SharedCrossfadeIdx != INDEX_NONE)
	// 			{
	// 				WireColor.R = (TransNode->SharedCrossfadeIdx & 1 ? 1.0f : 0.15f);
	// 				WireColor.G = (TransNode->SharedCrossfadeIdx & 2 ? 1.0f : 0.15f);
	// 				WireColor.B = (TransNode->SharedCrossfadeIdx & 4 ? 1.0f : 0.15f);
	// 			}


	// If shared transition, show different color
	if (TransNode->bSharedRules)
	{
		// Override alpha of shared colors as some have old 0.25f alpha serialized
		FLinearColor SharedColor(TransNode->SharedColor);
		SharedColor.A = 1.0f;
		BaseColor = SharedColor;
	}

	if (TransNode->bDisabled)
	{
		FLinearColor DisabledColor(BaseColor.GetSpecifiedColor());
		DisabledColor.A *= 0.5f;
		BaseColor = DisabledColor;
	}

	return bIsHovered ? HoverColor : BaseColor;
}

bool SGraphNodeAnimTransition::IsTransitionActive(int32 TransitionIndex, UAnimBlueprintGeneratedClass& AnimClass, UAnimationStateMachineGraph& StateMachineGraph, UAnimInstance& AnimInstance)
{
	if (AnimClass.GetAnimNodeProperties().Num())
	{
		if (FAnimNode_StateMachine* CurrentInstance = AnimClass.GetPropertyInstance<FAnimNode_StateMachine>(&AnimInstance, StateMachineGraph.OwnerAnimGraphNode))
		{
			if (CurrentInstance->IsTransitionActive(TransitionIndex))
			{
				return true;
			}
		}
	}
	return false;
}

FSlateColor SGraphNodeAnimTransition::GetTransitionColor() const
{	
	// Highlight the transition node when the node is hovered or when the previous state is hovered
	UAnimStateTransitionNode* TransNode = CastChecked<UAnimStateTransitionNode>(GraphNode);
	return StaticGetTransitionColor(TransNode, (IsHovered() || (PrevStateNodeWidgetPtr.IsValid() && PrevStateNodeWidgetPtr.Pin()->IsHovered())));
}

const FSlateBrush* SGraphNodeAnimTransition::GetTransitionIconImage() const
{
	UAnimStateTransitionNode* TransNode = CastChecked<UAnimStateTransitionNode>(GraphNode);
	return (TransNode->LogicType == ETransitionLogicType::TLT_Inertialization)
		? FAppStyle::GetBrush("Graph.AnimTransitionNode.Icon_Inertialization")
		: FAppStyle::GetBrush("Graph.AnimTransitionNode.Icon");
}

FString SGraphNodeAnimTransition::GetCurrentDuration() const
{
	UAnimStateTransitionNode* TransNode = CastChecked<UAnimStateTransitionNode>(GraphNode);

	return FString::Printf(TEXT("%.2f seconds"), TransNode->CrossfadeDuration);
}

void SGraphNodeAnimTransition::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	UAnimStateTransitionNode* TransNode = CastChecked<UAnimStateTransitionNode>(GraphNode);
	if (UEdGraphPin* Pin = TransNode->GetInputPin())
	{
		GetOwnerPanel()->AddPinToHoverSet(Pin);
	}

	SGraphNode::OnMouseEnter(MyGeometry, MouseEvent);
}

void SGraphNodeAnimTransition::OnMouseLeave(const FPointerEvent& MouseEvent)
{	
	UAnimStateTransitionNode* TransNode = CastChecked<UAnimStateTransitionNode>(GraphNode);
	if (UEdGraphPin* Pin = TransNode->GetInputPin())
	{
		GetOwnerPanel()->RemovePinFromHoverSet(Pin);
	}

	SGraphNode::OnMouseLeave(MouseEvent);
}

int32 SGraphNodeAnimTransition::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	// Boost layer ID to sort in front of states
	return SGraphNode::OnPaint(Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId + 100, InWidgetStyle, bParentEnabled);
}

void SGraphNodeAnimTransition::PositionBetweenTwoNodesWithOffset(const FGeometry& StartGeom, const FGeometry& EndGeom, int32 NodeIndex, int32 MaxNodes) const
{
	// Get a reasonable seed point (halfway between the boxes)
	const FVector2D StartCenter = FGeometryHelper::CenterOf(StartGeom);
	const FVector2D EndCenter = FGeometryHelper::CenterOf(EndGeom);
	const FVector2D SeedPoint = (StartCenter + EndCenter) * 0.5f;

	// Find the (approximate) closest points between the two boxes
	const FVector2D StartAnchorPoint = FGeometryHelper::FindClosestPointOnGeom(StartGeom, SeedPoint);
	const FVector2D EndAnchorPoint = FGeometryHelper::FindClosestPointOnGeom(EndGeom, SeedPoint);

	// Position ourselves halfway along the connecting line between the nodes, elevated away perpendicular to the direction of the line
	const float Height = 24.0f;

	const FVector2D DesiredNodeSize = GetDesiredSize();

	FVector2D DeltaPos(EndAnchorPoint - StartAnchorPoint);

	if (DeltaPos.IsNearlyZero())
	{
		DeltaPos = FVector2D(10.0f, 0.0f);
	}

	const FVector2D Normal = FVector2D(DeltaPos.Y, -DeltaPos.X).GetSafeNormal();

	const FVector2D NewCenter = StartAnchorPoint + (0.5f * DeltaPos) + (Height * Normal);

	FVector2D DeltaNormal = DeltaPos.GetSafeNormal();
	
	// Calculate node offset in the case of multiple transitions between the same two nodes
	// MultiNodeOffset: the offset where 0 is the centre of the transition, -1 is 1 <size of node>
	// towards the PrevStateNode and +1 is 1 <size of node> towards the NextStateNode.

	const float MutliNodeSpace = 0.0f; // Space between multiple transition nodes (in units of <size of node> )
	const float MultiNodeStep = (1.f + MutliNodeSpace); //Step between node centres (Size of node + size of node spacer)

	const float MultiNodeStart = -((MaxNodes - 1) * MultiNodeStep) / 2.f;
	const float MultiNodeOffset = MultiNodeStart + (NodeIndex * MultiNodeStep);

	// Now we need to adjust the new center by the node size, zoom factor and multi node offset
	const FVector2D NewCorner = NewCenter - (0.5f * DesiredNodeSize) + (DeltaNormal * MultiNodeOffset * DesiredNodeSize.X);

	GraphNode->NodePosX = static_cast<int32>(NewCorner.X);
	GraphNode->NodePosY = static_cast<int32>(NewCorner.Y);

	UAnimStateTransitionNode* TransNode = CastChecked<UAnimStateTransitionNode>(GraphNode);
	TransNode->CachedRotation = DeltaNormal;
}

const FSlateBrush* SGraphNodeAnimTransition::GetShadowBrush(bool bSelected) const
{
	return FAppStyle::GetBrush(TEXT("Graph.AnimTransitionNode.Shadow"));
}

/////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
