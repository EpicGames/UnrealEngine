// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/Vector2D.h"
#include "SGraphNode.h"
#include "SNodePanel.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

class STextEntryPopup;
class SToolTip;
class SWidget;
class UAnimStateTransitionNode;
class UObject;
struct FGeometry;
struct FPointerEvent;
struct FSlateBrush;

class SGraphNodeAnimTransition : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SGraphNodeAnimTransition){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UAnimStateTransitionNode* InNode);

	// SNodePanel::SNode interface
	virtual void GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const override;
	virtual void MoveTo(const FVector2f& NewPosition, FNodeSet& NodeFilter, bool bMarkDirty = true) override;
	virtual bool RequiresSecondPassLayout() const override;
	virtual void PerformSecondPassLayout(const TMap< UObject*, TSharedRef<SNode> >& NodeToWidgetLookup) const override;
	// End of SNodePanel::SNode interface

	// SGraphNode interface
	virtual void UpdateGraphNode() override;
	virtual TSharedPtr<SToolTip> GetComplexTooltip() override;
	virtual const FSlateBrush* GetShadowBrush(bool bSelected) const override;
	// End of SGraphNode interface

	// SWidget interface
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const;
	// End of SWidget interface

	// Calculate position for multiple nodes to be placed between a start and end point, by providing this nodes index and max expected nodes 
	void PositionBetweenTwoNodesWithOffset(const FGeometry& StartGeom, const FGeometry& EndGeom, int32 NodeIndex, int32 MaxNodes) const;

	static FSlateColor StaticGetTransitionColor(UAnimStateTransitionNode* TransNode, bool bIsHovered);

	static bool IsTransitionActive(int32 TransitionIndex, class UAnimBlueprintGeneratedClass& AnimClass, class UAnimationStateMachineGraph& StateMachineGraph, class UAnimInstance& AnimInstance);
private:
	TSharedPtr<STextEntryPopup> TextEntryWidget;

	/** Cache of the widget representing the previous state node */
	mutable TWeakPtr<SNode> PrevStateNodeWidgetPtr;

private:
	FText GetPreviewCornerText(bool reverse) const;
	FSlateColor GetTransitionColor() const;
	const FSlateBrush* GetTransitionIconImage() const;

	TSharedRef<SWidget> GenerateInlineDisplayOrEditingWidget(bool bShowGraphPreview);
	TSharedRef<SWidget> GenerateRichTooltip();

	FString GetCurrentDuration() const;
};
