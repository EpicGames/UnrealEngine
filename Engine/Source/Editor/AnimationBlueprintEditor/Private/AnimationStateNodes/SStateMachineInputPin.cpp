// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStateMachineInputPin.h"

#include "SGraphPanel.h"

void SStateMachineInputPin::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	this->SetCursor(EMouseCursor::Default);

	bShowLabel = false;

	GraphPinObj = InPin;
	check(GraphPinObj != nullptr);

	SetVisibility(EVisibility::HitTestInvisible);

	// Set up a brush-less border so this pin is effectively hidden
	SBorder::Construct( SBorder::FArguments()
		.BorderImage(nullptr)
	);
}

TSharedRef<SWidget>	SStateMachineInputPin::GetDefaultValueWidget()
{
	return SNullWidget::NullWidget;
}
