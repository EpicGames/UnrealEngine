// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStateMachineOutputPin.h"

#include "AnimStateAliasNode.h"
#include "AnimStateConduitNode.h"
#include "AnimStateEntryNode.h"
#include "AnimStateNode.h"
#include "SGraphPanel.h"

void SStateMachineOutputPin::Construct(const FArguments& InArgs, UEdGraphPin* InPin)
{
	this->SetCursor( EMouseCursor::Default );

	bShowLabel = true;

	GraphPinObj = InPin;
	check(GraphPinObj != NULL);

	const UEdGraphSchema* Schema = GraphPinObj->GetSchema();
	check(Schema);

	// Set up a hover for pins that is tinted the color of the pin.
	SBorder::Construct( SBorder::FArguments()
		.BorderImage( this, &SStateMachineOutputPin::GetPinBorder )
		.BorderBackgroundColor( this, &SStateMachineOutputPin::GetPinColor )
		.OnMouseButtonDown( this, &SStateMachineOutputPin::OnPinMouseDown )
		.Cursor( this, &SStateMachineOutputPin::GetPinCursor )
	);
}

TSharedRef<SWidget>	SStateMachineOutputPin::GetDefaultValueWidget()
{
	return SNew(STextBlock);
}

const FSlateBrush* SStateMachineOutputPin::GetPinBorder() const
{
	UEdGraphNode* Node = GraphPinObj->GetOwningNode();
	if (Node)
	{
		if (Node->IsA<UAnimStateAliasNode>())
		{
			return ( IsHovered() )
				? FAppStyle::GetBrush( TEXT("Graph.AnimAliasNode.Pin.BackgroundHovered") )
				: FAppStyle::GetBrush( TEXT("Graph.AnimStateNode.Pin.Background") );
		}
		else if (Node->IsA<UAnimStateConduitNode>() || Node->IsA<UAnimStateEntryNode>())
		{
			return ( IsHovered() )
				? FAppStyle::GetBrush( TEXT("Graph.AnimConduitNode.Pin.BackgroundHovered") )
				: FAppStyle::GetBrush( TEXT("Graph.AnimStateNode.Pin.Background") );
		}
	}

	bool bAliasHovered = false;
	if (UAnimStateNode* StateNode = Cast<UAnimStateNode>(Node))
	{
		const TSharedPtr<SGraphNode> OwnerNode = OwnerNodePtr.Pin();
		if(OwnerNode.IsValid())
		{
			if (UAnimStateAliasNode* HoveredAlias = Cast<UAnimStateAliasNode>(OwnerNode->GetOwnerPanel()->GetCurrentHoveredNode()))
			{
				if (HoveredAlias->bGlobalAlias || HoveredAlias->GetAliasedStates().Contains(StateNode))
				{
					bAliasHovered = true;
				}
			}
		}
	}

	return ( IsHovered() || bAliasHovered )
		? FAppStyle::GetBrush( TEXT("Graph.AnimStateNode.Pin.BackgroundHovered") )
		: FAppStyle::GetBrush( TEXT("Graph.AnimStateNode.Pin.Background") );
}