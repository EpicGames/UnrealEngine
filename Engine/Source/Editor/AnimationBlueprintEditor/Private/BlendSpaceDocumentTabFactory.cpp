// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlendSpaceDocumentTabFactory.h"

#include "AnimGraphNode_BlendSpaceGraphBase.h"
#include "AnimNodes/AnimNode_BlendSpaceGraphBase.h"
#include "Animation/AnimBlueprintGeneratedClass.h"
#include "Animation/BlendSpace.h"
#include "Animation/BlendSpace1D.h"
#include "AnimationBlendSpaceSampleGraph.h"
#include "AnimationBlueprintEditor.h"
#include "BlendSpaceGraph.h"
#include "BlueprintEditor.h"
#include "BlueprintEditorSettings.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/Map.h"
#include "Delegates/Delegate.h"
#include "EdGraph/EdGraph.h"
#include "Engine/Blueprint.h"
#include "Framework/Application/MenuStack.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Commands/UIAction.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Internationalization/Internationalization.h"
#include "Internationalization/Text.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Layout/Children.h"
#include "Layout/WidgetPath.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "Misc/AssertionMacros.h"
#include "Modules/ModuleManager.h"
#include "PersonaDelegates.h"
#include "PersonaModule.h"
#include "SGraphPreviewer.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "TabPayload_BlendSpaceGraph.h"
#include "Templates/Casts.h"
#include "Textures/SlateIcon.h"
#include "Toolkits/IToolkitHost.h"
#include "UObject/Class.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealNames.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SNullWidget.h"
#include "WorkflowOrientedApp/WorkflowTabManager.h"

class SWidget;
class UAnimSequence;
struct FSlateBrush;

static const FName BlendSpaceEditorID("BlendSpaceEditor");

#define LOCTEXT_NAMESPACE "FBlendSpaceDocumentTabFactory"

// Simple wrapper widget used to hold a reference to the graph document
class SBlendSpaceDocumentTab : public SCompoundWidget
{	
	SLATE_BEGIN_ARGS(SBlendSpaceDocumentTab) {}

	SLATE_DEFAULT_SLOT(FArguments, Content)
	
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UBlendSpaceGraph* InBlendSpaceGraph)
	{
		BlendSpaceGraph = InBlendSpaceGraph;
		
		ChildSlot
		[
			InArgs._Content.Widget
		];
	}

	TWeakObjectPtr<UBlendSpaceGraph> BlendSpaceGraph;
};

FBlendSpaceDocumentTabFactory::FBlendSpaceDocumentTabFactory(TSharedPtr<FAnimationBlueprintEditor> InBlueprintEditorPtr)
	: FDocumentTabFactory(BlendSpaceEditorID, InBlueprintEditorPtr)
	, BlueprintEditorPtr(InBlueprintEditorPtr)
{
}

TSharedRef<SWidget> FBlendSpaceDocumentTabFactory::CreateTabBody(const FWorkflowTabSpawnInfo& Info) const
{
	UBlendSpaceGraph* BlendSpaceGraph = FTabPayload_BlendSpaceGraph::GetBlendSpaceGraph(Info.Payload);

	FPersonaModule& PersonaModule = FModuleManager::LoadModuleChecked<FPersonaModule>("Persona");

	UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode = CastChecked<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceGraph->GetOuter());

	FBlendSpaceEditorArgs Args;

	Args.OnBlendSpaceCanvasDoubleClicked = FOnBlendSpaceCanvasDoubleClicked::CreateLambda([this, WeakBlendSpaceNode = TWeakObjectPtr<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceNode)]()
	{
		UBlueprintEditorSettings const* Settings = GetDefault<UBlueprintEditorSettings>();
		if (Settings->bDoubleClickNavigatesToParent)
		{
			if(BlueprintEditorPtr.IsValid() && WeakBlendSpaceNode.Get())
			{
				UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode = WeakBlendSpaceNode.Get();
				BlueprintEditorPtr.Pin()->JumpToHyperlink(BlendSpaceNode->GetOuter(), false);
			}
		}
	});

	Args.OnBlendSpaceNavigateUp = FOnBlendSpaceNavigateUp::CreateLambda([this, WeakBlendSpaceNode = TWeakObjectPtr<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceNode)]()
	{
		if(BlueprintEditorPtr.IsValid() && WeakBlendSpaceNode.Get())
		{
			UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode = WeakBlendSpaceNode.Get();
			BlueprintEditorPtr.Pin()->JumpToHyperlink(BlendSpaceNode->GetOuter(), false);
		}
	});

	Args.OnBlendSpaceNavigateDown = FOnBlendSpaceNavigateDown::CreateLambda([this, WeakBlendSpaceNode = TWeakObjectPtr<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceNode)]()
	{
		if(BlueprintEditorPtr.IsValid() && WeakBlendSpaceNode.Get())
		{
			UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode = WeakBlendSpaceNode.Get();
			TArrayView<UEdGraph* const> Graphs = BlendSpaceNode->GetGraphs();
			if (Graphs.Num() > 1)
			{
				// Display a child jump list
				FMenuBuilder MenuBuilder(true, nullptr);
				MenuBuilder.BeginSection("NavigateToGraph", LOCTEXT("ChildGraphPickerDesc", "Navigate to graph"));

				TArray<const UEdGraph*> SortedGraphs(Graphs);
				SortedGraphs.Sort([](const UEdGraph& A, const UEdGraph& B) { return FBlueprintEditor::GetGraphDisplayName(&A).CompareToCaseIgnored(FBlueprintEditor::GetGraphDisplayName(&B)) < 0; });

				for (const UEdGraph* Graph : SortedGraphs)
				{
					MenuBuilder.AddMenuEntry(
						BlueprintEditorPtr.Pin()->GetGraphDisplayName(Graph),
						LOCTEXT("ChildGraphPickerTooltip", "Pick the graph to enter"),
						FSlateIcon(),
						FUIAction(
							FExecuteAction::CreateLambda(
								[this,Graph]() 
								{
									BlueprintEditorPtr.Pin()->OpenDocument(Graph, FDocumentTracker::NavigatingCurrentDocument);
								}),
							FCanExecuteAction()));
				}
				MenuBuilder.EndSection();

				FSlateApplication::Get().PushMenu( 
					BlueprintEditorPtr.Pin()->GetToolkitHost()->GetParentWidget(),
					FWidgetPath(),
					MenuBuilder.MakeWidget(),
					FSlateApplication::Get().GetCursorPos(), // summon location
					FPopupTransitionEffect( FPopupTransitionEffect::TypeInPopup )
				);
			}
			else if (Graphs.Num() == 1)
			{
				BlueprintEditorPtr.Pin()->JumpToHyperlink(Graphs[0], false);
			}
		}
	});

	Args.OnBlendSpaceSampleDoubleClicked = FOnBlendSpaceSampleDoubleClicked::CreateLambda([this, WeakBlendSpaceNode = TWeakObjectPtr<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceNode)](int32 InSampleIndex)
	{
		if(BlueprintEditorPtr.IsValid() && WeakBlendSpaceNode.Get())
		{
			UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode = WeakBlendSpaceNode.Get();
			if(BlendSpaceNode->GetGraphs().IsValidIndex(InSampleIndex))
			{
				BlueprintEditorPtr.Pin()->JumpToHyperlink(BlendSpaceNode->GetGraphs()[InSampleIndex], false);
			}
		}
	});

	Args.OnBlendSpaceSampleAdded = FOnBlendSpaceSampleAdded::CreateLambda([this, WeakBlendSpaceNode = TWeakObjectPtr<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceNode)](UAnimSequence* InSequence, const FVector& InSamplePoint, bool bRunAnalysis) -> int32
	{
		int32 Index = INDEX_NONE;
		if(WeakBlendSpaceNode.Get())
		{
			UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode = WeakBlendSpaceNode.Get();
			UAnimationBlendSpaceSampleGraph* NewGraph = BlendSpaceNode->AddGraph(TEXT("NewSample"), InSequence);
			Index = BlendSpaceNode->GetSampleIndex(NewGraph);
			BlueprintEditorPtr.Pin()->RefreshMyBlueprint();
			BlueprintEditorPtr.Pin()->RenameNewlyAddedAction(NewGraph->GetFName());
		}
		return Index;
	});

	Args.OnBlendSpaceSampleRemoved = FOnBlendSpaceSampleRemoved::CreateLambda([this, WeakBlendSpaceNode = TWeakObjectPtr<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceNode)](int32 InSampleIndex)
	{
		if(WeakBlendSpaceNode.Get())
		{
			UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode = WeakBlendSpaceNode.Get();
			BlendSpaceNode->RemoveGraph(InSampleIndex);
			BlueprintEditorPtr.Pin()->RefreshMyBlueprint();
		}
	});

	Args.OnBlendSpaceSampleReplaced = FOnBlendSpaceSampleReplaced::CreateLambda([this, WeakBlendSpaceNode = TWeakObjectPtr<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceNode)](int32 InSampleIndex, UAnimSequence* InSequence)
	{
		if(WeakBlendSpaceNode.Get())
		{
			UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode = WeakBlendSpaceNode.Get();
			BlendSpaceNode->ReplaceGraph(InSampleIndex, InSequence);
			BlueprintEditorPtr.Pin()->RefreshMyBlueprint();
		}
	});

	Args.OnGetBlendSpaceSampleName = FOnGetBlendSpaceSampleName::CreateLambda([this, WeakBlendSpaceNode = TWeakObjectPtr<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceNode)](int32 InSampleIndex) -> FName
	{
		if(WeakBlendSpaceNode.Get())
		{
			UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode = WeakBlendSpaceNode.Get();
			return BlendSpaceNode->GetGraphs()[InSampleIndex]->GetFName();
		}

		return NAME_None;
	});

	Args.OnExtendSampleTooltip = FOnExtendBlendSpaceSampleTooltip::CreateLambda([this, WeakBlendSpaceNode = TWeakObjectPtr<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceNode)](int32 InSampleIndex) -> TSharedRef<SWidget>
	{
		if(WeakBlendSpaceNode.Get())
		{
			UAnimGraphNode_BlendSpaceGraphBase* BlendSpaceNode = WeakBlendSpaceNode.Get();
			if(BlendSpaceNode->GetGraphs().IsValidIndex(InSampleIndex))
			{
				return SNew(SGraphPreviewer, BlendSpaceNode->GetGraphs()[InSampleIndex])
					.CornerOverlayText(LOCTEXT("SampleGraphOverlay", "ANIMATION"))
					.ShowGraphStateOverlay(false);
			}
		}

		return SNullWidget::NullWidget;
	});

	Args.PreviewPosition = MakeAttributeLambda([this, WeakBlendSpaceNode = TWeakObjectPtr<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceNode)]()
	{
		if(WeakBlendSpaceNode.Get())
		{
			if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(WeakBlendSpaceNode.Get()))
			{
				if (UObject* ActiveObject = Blueprint->GetObjectBeingDebugged())
				{
					if (UAnimBlueprintGeneratedClass* Class = Cast<UAnimBlueprintGeneratedClass>(ActiveObject->GetClass()))
					{
						if(int32* NodeIndexPtr = Class->GetAnimBlueprintDebugData().NodePropertyToIndexMap.Find(WeakBlendSpaceNode))
						{
							int32 AnimNodeIndex = *NodeIndexPtr;
							// reverse node index temporarily because of a bug in NodeGuidToIndexMap
							AnimNodeIndex = Class->GetAnimNodeProperties().Num() - AnimNodeIndex - 1;

							if (FAnimBlueprintDebugData::FBlendSpacePlayerRecord* DebugInfo = Class->GetAnimBlueprintDebugData().BlendSpacePlayerRecordsThisFrame.FindByPredicate(
								[AnimNodeIndex](const FAnimBlueprintDebugData::FBlendSpacePlayerRecord& InRecord){ return InRecord.NodeID == AnimNodeIndex; }))
							{
								return DebugInfo->Position;
							}
						}
					}
				}
			}
		}

		return FVector::ZeroVector;
	});

	Args.PreviewFilteredPosition = MakeAttributeLambda([this, WeakBlendSpaceNode = TWeakObjectPtr<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceNode)]()
	{
		if (WeakBlendSpaceNode.Get())
		{
			if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(WeakBlendSpaceNode.Get()))
			{
				if (UObject* ActiveObject = Blueprint->GetObjectBeingDebugged())
				{
					if (UAnimBlueprintGeneratedClass* Class = Cast<UAnimBlueprintGeneratedClass>(ActiveObject->GetClass()))
					{
						if (int32* NodeIndexPtr = Class->GetAnimBlueprintDebugData().NodePropertyToIndexMap.Find(WeakBlendSpaceNode))
						{
							int32 AnimNodeIndex = *NodeIndexPtr;
							// reverse node index temporarily because of a bug in NodeGuidToIndexMap
							AnimNodeIndex = Class->GetAnimNodeProperties().Num() - AnimNodeIndex - 1;

							if (FAnimBlueprintDebugData::FBlendSpacePlayerRecord* DebugInfo = Class->GetAnimBlueprintDebugData().BlendSpacePlayerRecordsThisFrame.FindByPredicate(
								[AnimNodeIndex](const FAnimBlueprintDebugData::FBlendSpacePlayerRecord& InRecord) { return InRecord.NodeID == AnimNodeIndex; }))
							{
								return DebugInfo->FilteredPosition;
							}
						}
					}
				}
			}
		}

		return FVector::ZeroVector;
	});

	Args.OnSetPreviewPosition = FOnSetBlendSpacePreviewPosition::CreateLambda([this, WeakBlendSpaceNode = TWeakObjectPtr<UAnimGraphNode_BlendSpaceGraphBase>(BlendSpaceNode)](FVector InPreviewPosition)
	{
		if(WeakBlendSpaceNode.Get())
		{
			if (UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForNode(WeakBlendSpaceNode.Get()))
			{
				if (UObject* ActiveObject = Blueprint->GetObjectBeingDebugged())
				{
					if (UAnimBlueprintGeneratedClass* Class = Cast<UAnimBlueprintGeneratedClass>(ActiveObject->GetClass()))
					{
						if(FAnimNode_BlendSpaceGraphBase* BlendSpaceGraphNode = Class->GetPropertyInstance<FAnimNode_BlendSpaceGraphBase>(ActiveObject, WeakBlendSpaceNode.Get()))
						{
							BlendSpaceGraphNode->SetPreviewPosition(InPreviewPosition);
						}
					}
				}
			}
		}
	});

	Args.StatusBarName = TEXT("AssetEditor.AnimationBlueprintEditor.MainMenu");

	return
		SNew(SBlendSpaceDocumentTab, BlendSpaceGraph)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			[
				BlueprintEditorPtr.Pin()->CreateGraphTitleBarWidget(Info.TabInfo.ToSharedRef(), BlendSpaceGraph)
			]
			+SVerticalBox::Slot()
			.FillHeight(1.0f)
			[
				PersonaModule.CreateBlendSpaceEditWidget(BlendSpaceGraph->BlendSpace, Args)
			]
		];
}

const FSlateBrush* FBlendSpaceDocumentTabFactory::GetTabIcon(const FWorkflowTabSpawnInfo& Info) const
{
	UBlendSpaceGraph* BlendSpaceGraph = FTabPayload_BlendSpaceGraph::GetBlendSpaceGraph(Info.Payload);

	if (UBlendSpace1D* BlendSpace1D = Cast<UBlendSpace1D>(BlendSpaceGraph->BlendSpace))
	{
		return FAppStyle::GetBrush("ClassIcon.BlendSpace1D");
	}
	else
	{
		return FAppStyle::GetBrush("ClassIcon.BlendSpace");
	}
}

bool FBlendSpaceDocumentTabFactory::IsPayloadSupported(TSharedRef<FTabPayload> Payload) const
{
	return (Payload->PayloadType == UBlendSpaceGraph::StaticClass()->GetFName() && Payload->IsValid());
}

bool FBlendSpaceDocumentTabFactory::IsPayloadValid(TSharedRef<FTabPayload> Payload) const
{
	if (Payload->PayloadType == UBlendSpaceGraph::StaticClass()->GetFName())
	{
		return Payload->IsValid();
	}
	return false;
}

TAttribute<FText> FBlendSpaceDocumentTabFactory::ConstructTabName(const FWorkflowTabSpawnInfo& Info) const
{
	check(Info.Payload.IsValid());

	UBlendSpaceGraph* BlendSpaceGraph = FTabPayload_BlendSpaceGraph::GetBlendSpaceGraph(Info.Payload);

	return MakeAttributeLambda([WeakBlendSpace = TWeakObjectPtr<UBlendSpace>(BlendSpaceGraph->BlendSpace)]()
		{ 
			return WeakBlendSpace.Get() ? FText::FromName(WeakBlendSpace->GetFName()) : FText::GetEmpty();
		});
}

void FBlendSpaceDocumentTabFactory::OnTabActivated(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<SBlendSpaceDocumentTab> BlendSpaceDocumentTab = StaticCastSharedRef<SBlendSpaceDocumentTab>(Tab->GetContent());
	if(UBlendSpaceGraph* BlendSpaceGraph = BlendSpaceDocumentTab->BlendSpaceGraph.Get())
	{
		if(TSharedPtr<FAnimationBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin())
		{
			BlueprintEditor->SetDetailObject(BlendSpaceGraph);
		}
	}
}

void FBlendSpaceDocumentTabFactory::OnTabForegrounded(TSharedPtr<SDockTab> Tab) const
{
	TSharedRef<SBlendSpaceDocumentTab> BlendSpaceDocumentTab = StaticCastSharedRef<SBlendSpaceDocumentTab>(Tab->GetContent());
	if(UBlendSpaceGraph* BlendSpaceGraph = BlendSpaceDocumentTab->BlendSpaceGraph.Get())
	{
		if(TSharedPtr<FAnimationBlueprintEditor> BlueprintEditor = BlueprintEditorPtr.Pin())
		{
			BlueprintEditor->SetDetailObject(BlendSpaceGraph);
		}
	}
}

#undef LOCTEXT_NAMESPACE