// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Attribute.h"
#include "Templates/SharedPointer.h"
#include "WorkflowOrientedApp/WorkflowTabFactory.h"

class FAnimationBlueprintEditor;
class FText;
class SDockTab;
class SWidget;
struct FSlateBrush;

struct FBlendSpaceDocumentTabFactory : public FDocumentTabFactory
{
public:
	FBlendSpaceDocumentTabFactory(TSharedPtr<FAnimationBlueprintEditor> InBlueprintEditorPtr);

protected:
	// FWorkflowTabFactory interface
	virtual TSharedRef<SWidget> CreateTabBody(const FWorkflowTabSpawnInfo& Info) const override;
	virtual const FSlateBrush* GetTabIcon(const FWorkflowTabSpawnInfo& Info) const override;
	virtual bool IsPayloadSupported(TSharedRef<FTabPayload> Payload) const override;
	virtual bool IsPayloadValid(TSharedRef<FTabPayload> Payload) const override;
	virtual TAttribute<FText> ConstructTabName(const FWorkflowTabSpawnInfo& Info) const override;
	virtual void OnTabActivated(TSharedPtr<SDockTab> Tab) const override;
	virtual void OnTabForegrounded(TSharedPtr<SDockTab> Tab) const override;
	
protected:
	TWeakPtr<FAnimationBlueprintEditor> BlueprintEditorPtr;
};
