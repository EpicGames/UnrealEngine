// Copyright Epic Games, Inc. All Rights Reserved.

#include "StateMachineConnectionDrawingPolicy.h"

#include "AnimStateEntryNode.h"
#include "AnimStateNode.h"
#include "AnimStateNodeBase.h"
#include "AnimStateTransitionNode.h"
#include "AnimationStateNodes/SGraphNodeAnimTransition.h"
#include "Containers/Set.h"
#include "EdGraph/EdGraphPin.h"
#include "Layout/ArrangedChildren.h"
#include "Layout/ArrangedWidget.h"
#include "Layout/PaintGeometry.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "Misc/Optional.h"
#include "Rendering/DrawElements.h"
#include "Rendering/RenderingCommon.h"
#include "SGraphNode.h"
#include "SGraphPin.h"
#include "Styling/SlateBrush.h"
#include "Templates/Casts.h"
#include "Math/UnrealMathUtility.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/StyleColors.h"

class FSlateRect;
class SWidget;
struct FGeometry;

/////////////////////////////////////////////////////
// FStateMachineConnectionDrawingPolicy

FStateMachineConnectionDrawingPolicy::FStateMachineConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj)
	: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, ZoomFactor, InClippingRect, InDrawElements)
	, GraphObj(InGraphObj)
{
	ArrowImage = FAppStyle::GetBrush( TEXT("Graph.AnimStateNode.ConnectionArrow") );
}

void FStateMachineConnectionDrawingPolicy::DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params)
{
	Params.AssociatedPin1 = OutputPin;
	Params.AssociatedPin2 = InputPin;
	Params.WireThickness = 1.5f;

	if (InputPin)
	{
		if (UAnimStateTransitionNode* TransNode = Cast<UAnimStateTransitionNode>(InputPin->GetOwningNode()))
		{
			const bool IsInputPinHovered = HoveredPins.Contains(InputPin);
			Params.WireColor = SGraphNodeAnimTransition::StaticGetTransitionColor(TransNode, IsInputPinHovered).GetSpecifiedColor();
			Params.bUserFlag1 = TransNode->Bidirectional;
		}
	}

	const bool bDeemphasizeUnhoveredPins = HoveredPins.Num() > 0;
	if (bDeemphasizeUnhoveredPins && !Params.bUserFlag2)
	{
		ApplyHoverDeemphasis(OutputPin, InputPin, /*inout*/ Params.WireThickness, /*inout*/ Params.WireColor);
	}

	// Make the transition that is currently relinked, semi-transparent.
	if (Params.bUserFlag2)
	{
		Params.WireColor.A = 0.5f;
	}
}

void FStateMachineConnectionDrawingPolicy::DetermineLinkGeometry(
	FArrangedChildren& ArrangedNodes, 
	TSharedRef<SWidget>& OutputPinWidget,
	UEdGraphPin* OutputPin,
	UEdGraphPin* InputPin,
	/*out*/ FArrangedWidget*& StartWidgetGeometry,
	/*out*/ FArrangedWidget*& EndWidgetGeometry
	)
{
	if (UAnimStateEntryNode* EntryNode = Cast<UAnimStateEntryNode>(OutputPin->GetOwningNode()))
	{
		StartWidgetGeometry = PinGeometries->Find(OutputPinWidget);

		UAnimStateNodeBase* State = CastChecked<UAnimStateNodeBase>(InputPin->GetOwningNode());
		int32 StateIndex = NodeWidgetMap.FindChecked(State);
		EndWidgetGeometry = &(ArrangedNodes[StateIndex]);
	}
	else if (UAnimStateTransitionNode* TransNode = Cast<UAnimStateTransitionNode>(InputPin->GetOwningNode()))
	{
		UAnimStateNodeBase* PrevState = TransNode->GetPreviousState();
		UAnimStateNodeBase* NextState = TransNode->GetNextState();
		if ((PrevState != NULL) && (NextState != NULL))
		{
			int32* PrevNodeIndex = NodeWidgetMap.Find(PrevState);
			int32* NextNodeIndex = NodeWidgetMap.Find(NextState);
			if ((PrevNodeIndex != NULL) && (NextNodeIndex != NULL))
			{
				StartWidgetGeometry = &(ArrangedNodes[*PrevNodeIndex]);
				EndWidgetGeometry = &(ArrangedNodes[*NextNodeIndex]);
			}
		}
	}
}

void FStateMachineConnectionDrawingPolicy::Draw(TMap<TSharedRef<SWidget>, FArrangedWidget>& InPinGeometries, FArrangedChildren& ArrangedNodes)
{
	// Build an acceleration structure to quickly find geometry for the nodes
	NodeWidgetMap.Empty();
	for (int32 NodeIndex = 0; NodeIndex < ArrangedNodes.Num(); ++NodeIndex)
	{
		FArrangedWidget& CurWidget = ArrangedNodes[NodeIndex];
		TSharedRef<SGraphNode> ChildNode = StaticCastSharedRef<SGraphNode>(CurWidget.Widget);
		NodeWidgetMap.Add(ChildNode->GetNodeObj(), NodeIndex);
	}

	// Now draw
	FConnectionDrawingPolicy::Draw(InPinGeometries, ArrangedNodes);
}

void FStateMachineConnectionDrawingPolicy::DrawPreviewConnector(const FGeometry& PinGeometry, const FVector2f& StartPoint, const FVector2f& EndPoint, UEdGraphPin* Pin)
{
	FConnectionParams Params;
	Params.bUserFlag2 = true;	// bUserFlag2 is used to indicate whether the drawn arrow is a preview transition (the temporary transition when creating or relinking).
	UEdGraphPin* InputPin = Pin->Direction == EGPD_Input ? Pin : nullptr;
	UEdGraphPin* OutputPin = Pin->Direction == EGPD_Output ? Pin : nullptr;
	DetermineWiringStyle(InputPin, OutputPin, /*inout*/ Params);

	const FVector2f SeedPoint = Pin->Direction == EGPD_Input ? StartPoint : EndPoint;
	const FVector2f AdjustedSeedPoint = FGeometryHelper::FindClosestPointOnGeom(PinGeometry, SeedPoint);
	const FVector2f AdjustedStartPoint = Pin->Direction == EGPD_Input ? StartPoint : AdjustedSeedPoint;
	const FVector2f AdjustedEndPoint = Pin->Direction == EGPD_Input ? AdjustedSeedPoint : EndPoint;

	DrawSplineWithArrow(AdjustedStartPoint, AdjustedEndPoint, Params);
}


void FStateMachineConnectionDrawingPolicy::DrawSplineWithArrow(const FVector2f& StartAnchorPoint, const FVector2f& EndAnchorPoint, const FConnectionParams& Params)
{
	Internal_DrawLineWithArrow(StartAnchorPoint, EndAnchorPoint, Params);

	// Is the connection bidirectional?
	if (Params.bUserFlag1)
	{
		Internal_DrawLineWithArrow(EndAnchorPoint, StartAnchorPoint, Params);
	}
}

void FStateMachineConnectionDrawingPolicy::Internal_DrawLineWithArrow(const FVector2f& StartAnchorPoint, const FVector2f& EndAnchorPoint, const FConnectionParams& Params)
{
	const float LineSeparationAmount = 6.0f * ZoomFactor;

	const FVector2f DeltaPos = EndAnchorPoint - StartAnchorPoint;
	const FVector2f UnitDelta = DeltaPos.GetSafeNormal();
	const FVector2f Normal = FVector2f(DeltaPos.Y, -DeltaPos.X).GetSafeNormal();

	// Come up with the final start/end points
	const FVector2f DirectionBias = Normal * LineSeparationAmount;
	const FVector2f LengthBias = ArrowRadius.X * UnitDelta;
	const FVector2f StartPoint = StartAnchorPoint + DirectionBias + LengthBias;
	const FVector2f EndPoint = EndAnchorPoint + DirectionBias - LengthBias;
	FLinearColor ArrowHeadColor = Params.WireColor;

	// Draw a line/spline.
	// LengthBias * 0.8f is to ensure the line doesnt overlap the arrowhead glyph so they dont show up overlapping when semi-transparent
	DrawConnection(WireLayerID, StartPoint, EndPoint - (LengthBias * 0.8f), Params);

	const FVector2f ArrowDrawPos = EndPoint - ArrowRadius;
	const double AngleInRadians = FMath::Atan2(DeltaPos.Y, DeltaPos.X);

	// Draw the transition grab handles in case the mouse is hovering the transition
	bool bStartHovered = false;
	bool bEndHovered = false;
	const FVector2f FVecMousePos = FVector2f(AbsoluteMousePosition.X, AbsoluteMousePosition.Y);
	const FVector2f ClosestPoint = FMath::ClosestPointOnSegment2D(FVecMousePos, StartPoint, EndPoint);
	if ((ClosestPoint - FVecMousePos).Length() < RelinkHandleHoverRadius * ZoomFactor)
	{
		bStartHovered = (StartPoint - AbsoluteMousePosition).Length() < RelinkHandleHoverRadius * ZoomFactor;
		bEndHovered = (EndPoint - AbsoluteMousePosition).Length() < RelinkHandleHoverRadius * ZoomFactor;
		FVector2f HoverIndicatorPosition = bStartHovered ? StartPoint : EndPoint;

		// Set the hovered pin results. This will be used by the SGraphPanel again.
		const float SquaredDistToPin1 = (Params.AssociatedPin1 != nullptr) ? (StartPoint - AbsoluteMousePosition).SizeSquared() : FLT_MAX;
		const float SquaredDistToPin2 = (Params.AssociatedPin2 != nullptr) ? (EndPoint - AbsoluteMousePosition).SizeSquared() : FLT_MAX;
		UEdGraphPin* Pin1 = Params.AssociatedPin1;
		UEdGraphPin* Pin2 = Params.AssociatedPin2;
		if (Pin2)
		{
			// Forward transition links to the destination state node's output pin as only it has a valid pin widget
			if (UAnimStateTransitionNode* TransitionNode = Cast<UAnimStateTransitionNode>(Pin2->GetOwningNode()))
			{
				Pin2 = TransitionNode->GetNextState()->GetInputPin();
			}
		}

		if (bStartHovered)
		{
			SplineOverlapResult = FGraphSplineOverlapResult(Pin2, Pin1, FMath::Min(SquaredDistToPin1, SquaredDistToPin2), SquaredDistToPin2, SquaredDistToPin1, true);
		}
		else if (bEndHovered)
		{
			SplineOverlapResult = FGraphSplineOverlapResult(Pin1, Pin2, FMath::Min(SquaredDistToPin1, SquaredDistToPin2), SquaredDistToPin1, SquaredDistToPin2, true);
		}

		// Draw grab handles only in case no relinking operation is performed
		if (RelinkConnections.IsEmpty() && !Params.bUserFlag2)
		{
			if (bStartHovered)
			{
				// Draw solid selection circle behind the arrow head in case the arrow head is hovered (area that enables a relink).
				static FSlateRoundedBoxBrush RoundedBoxBrush = FSlateRoundedBoxBrush(FStyleColors::Foreground, ArrowImage->ImageSize.X * 0.5f);

				FSlateDrawElement::MakeBox(DrawElementsList,
					ArrowLayerID-1, // Draw behind the arrow
					FPaintGeometry(StartPoint - ArrowRadius + (LengthBias * 0.2f), ArrowImage->ImageSize * ZoomFactor, ZoomFactor),
					&RoundedBoxBrush,
					ESlateDrawEffect::None,
			FStyleColors::AccentOrange.GetSpecifiedColor());
			}
			else if (bEndHovered)
			{
				// Draw solid selection circle behind the arrow head in case the arrow head is hovered (area that enables a relink).
				static FSlateRoundedBoxBrush RoundedBoxBrush = FSlateRoundedBoxBrush(FStyleColors::Foreground, ArrowImage->ImageSize.X * 0.5f);

				FSlateDrawElement::MakeBox(DrawElementsList,
					ArrowLayerID-1, // Draw behind the arrow
					FPaintGeometry(EndPoint - ArrowRadius - (LengthBias * 0.2f), ArrowImage->ImageSize * ZoomFactor, ZoomFactor),
					&RoundedBoxBrush,
					ESlateDrawEffect::None,
					FStyleColors::AccentOrange.GetSpecifiedColor());

				ArrowHeadColor = FLinearColor::Black;
			}
		}
	}

	// Draw the number of relinked transitions on the preview transition.
	if (!RelinkConnections.IsEmpty() && Params.bUserFlag2)
	{
		// Get the number of actually relinked transitions.
		int32 NumRelinkedTransitions = 0;
		for (const FRelinkConnection& Connection : RelinkConnections)
		{
			NumRelinkedTransitions += UAnimStateTransitionNode::GetListTransitionNodesToRelink(Connection.SourcePin, Connection.TargetPin, SelectedGraphNodes).Num();

			if (UAnimStateEntryNode* EntryNode = Cast<UAnimStateEntryNode>(Connection.SourcePin->GetOwningNode()))
			{
				NumRelinkedTransitions += 1;
			}
		}

		const FVector2f TransitionCenter = StartAnchorPoint + DeltaPos * 0.5f;
		const FVector2f TextPosition = TransitionCenter + Normal * 15.0f * ZoomFactor;

		FSlateDrawElement::MakeText(
			DrawElementsList,
			ArrowLayerID,
			FPaintGeometry(TextPosition, ArrowImage->ImageSize * ZoomFactor, ZoomFactor),
			FText::AsNumber(NumRelinkedTransitions),
			FCoreStyle::Get().GetFontStyle("SmallFont"));
	}

	// Draw the transition arrow triangle
	FSlateDrawElement::MakeRotatedBox(
		DrawElementsList,
		ArrowLayerID,
		FPaintGeometry(ArrowDrawPos, ArrowImage->ImageSize * ZoomFactor, ZoomFactor),
		ArrowImage,
		ESlateDrawEffect::None,
		static_cast<float>(AngleInRadians),
		TOptional<FVector2f>(),
		FSlateDrawElement::RelativeToElement,
		ArrowHeadColor
	);
}

void FStateMachineConnectionDrawingPolicy::DrawCircle(const FVector2f& Center, float Radius, const FLinearColor& Color, const int NumLineSegments)
{
	TempPoints.Empty();

	const float NumFloatLineSegments = (float)NumLineSegments;
	for (int i = 0; i <= NumLineSegments; i++)
	{
		const float Angle = (i / NumFloatLineSegments) * TWO_PI;

		FVector2f PointOnCircle;
		PointOnCircle.X = cosf(Angle) * Radius;
		PointOnCircle.Y = sinf(Angle) * Radius;
		TempPoints.Add(PointOnCircle);
	}

	FSlateDrawElement::MakeLines(
		DrawElementsList,
		ArrowLayerID + 1,
		FPaintGeometry(Center, FVector2f(Radius, Radius) * ZoomFactor, ZoomFactor),
		TempPoints,
		ESlateDrawEffect::None,
		Color
	);
}

void FStateMachineConnectionDrawingPolicy::DrawSplineWithArrow(const FGeometry& StartGeom, const FGeometry& EndGeom, const FConnectionParams& Params)
{
	// Get a reasonable seed point (halfway between the boxes)
	const FVector2f StartCenter = FGeometryHelper::CenterOf(StartGeom);
	const FVector2f EndCenter = FGeometryHelper::CenterOf(EndGeom);
	const FVector2f SeedPoint = (StartCenter + EndCenter) * 0.5f;
	
	// Find the (approximate) closest points between the two boxes
	const FVector2f StartAnchorPoint = FGeometryHelper::FindClosestPointOnGeom(StartGeom, SeedPoint);
	const FVector2f EndAnchorPoint = FGeometryHelper::FindClosestPointOnGeom(EndGeom, SeedPoint);

	DrawSplineWithArrow(StartAnchorPoint, EndAnchorPoint, Params);
}

FVector2f FStateMachineConnectionDrawingPolicy::ComputeSplineTangent(const FVector2f& Start, const FVector2f& End) const
{
	const FVector2f Delta = End - Start;
	const FVector2f NormDelta = Delta.GetSafeNormal();

	return NormDelta;
}
