// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ConnectionDrawingPolicy.h"
#include "Containers/Map.h"
#include "HAL/Platform.h"
#include "Math/Vector2D.h"
#include "Templates/SharedPointer.h"

class FArrangedChildren;
class FArrangedWidget;
class FSlateRect;
class FSlateWindowElementList;
class SWidget;
class UEdGraph;
class UEdGraphNode;
class UEdGraphPin;
struct FGeometry;

/////////////////////////////////////////////////////
// FStateMachineConnectionDrawingPolicy

// This class draws the connections for an UEdGraph with a state machine schema
class FStateMachineConnectionDrawingPolicy : public FConnectionDrawingPolicy
{
protected:
	UEdGraph* GraphObj;

	TMap<UEdGraphNode*, int32> NodeWidgetMap;
public:
	// @Note FConnectionParams.bUserFlag2: Is used here to indicate whether the drawn arrow is a preview transition or a real one.

	//
	FStateMachineConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float ZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements, UEdGraph* InGraphObj);

	// FConnectionDrawingPolicy interface
	virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override;
	virtual void Draw(TMap<TSharedRef<SWidget>, FArrangedWidget>& PinGeometries, FArrangedChildren& ArrangedNodes) override;
	virtual void DetermineLinkGeometry(
		FArrangedChildren& ArrangedNodes, 
		TSharedRef<SWidget>& OutputPinWidget,
		UEdGraphPin* OutputPin,
		UEdGraphPin* InputPin,
		/*out*/ FArrangedWidget*& StartWidgetGeometry,
		/*out*/ FArrangedWidget*& EndWidgetGeometry
		) override;
	virtual void DrawSplineWithArrow(const FGeometry& StartGeom, const FGeometry& EndGeom, const FConnectionParams& Params) override;
	virtual void DrawSplineWithArrow(const FVector2f& StartPoint, const FVector2f& EndPoint, const FConnectionParams& Params) override;
	virtual void DrawPreviewConnector(const FGeometry& PinGeometry, const FVector2f& StartPoint, const FVector2f& EndPoint, UEdGraphPin* Pin) override;
	virtual FVector2f ComputeSplineTangent(const FVector2f& Start, const FVector2f& End) const override;
	// End of FConnectionDrawingPolicy interface

protected:
	void Internal_DrawLineWithArrow(const FVector2f& StartAnchorPoint, const FVector2f& EndAnchorPoint, const FConnectionParams& Params);

	// Draw line-based circle (no solid filling)
	void DrawCircle(const FVector2f& Center, float Radius, const FLinearColor& Color, const int NumLineSegments);
	TArray<FVector2f> TempPoints;

	static constexpr float RelinkHandleHoverRadius = 10.0f;
};
