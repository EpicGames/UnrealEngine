// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "WorkflowOrientedApp/WorkflowUObjectDocuments.h"
#include "BlendSpaceGraph.h"

// Tab payload used to generate blendspace documents
struct FTabPayload_BlendSpaceGraph : public FTabPayload
{
public:
	// Create a new payload wrapper for a UObject
	static TSharedRef<FTabPayload_BlendSpaceGraph> Make(const UBlendSpaceGraph* DocumentID)
	{
		return MakeShareable(new FTabPayload_BlendSpaceGraph(const_cast<UBlendSpaceGraph*>(DocumentID)));
	}

	// Determine if another payload is the same as this one
	virtual bool IsEqual(const TSharedRef<FTabPayload>& OtherPayload) const override
	{
		if (OtherPayload->PayloadType == PayloadType)
		{
			return this->DocumentID.HasSameIndexAndSerialNumber(StaticCastSharedRef<FTabPayload_BlendSpaceGraph>(OtherPayload)->DocumentID);
		}
		else if(OtherPayload->PayloadType == NAME_Object)
		{
			TWeakObjectPtr<UObject> WeakObject = FTabPayload_UObject::CastChecked<UObject>(OtherPayload);
			return WeakObject.HasSameIndexAndSerialNumber(DocumentID);
		}

		return false;
	}

	virtual bool IsValid() const override
	{
		return DocumentID.IsValid();
	}

	virtual ~FTabPayload_BlendSpaceGraph() = default;

	// Access the internal graph payload
	static UBlendSpaceGraph* GetBlendSpaceGraph(TSharedPtr<FTabPayload> Payload)
	{
		check((Payload.IsValid()) && (Payload->PayloadType == UBlendSpaceGraph::StaticClass()->GetFName()));

		return StaticCastSharedPtr<FTabPayload_BlendSpaceGraph>(Payload)->DocumentID.Get(true);
	}

private:
	// Buried constructor. Use Make instead!
	FTabPayload_BlendSpaceGraph(UBlendSpaceGraph* InDocumentID)
		: FTabPayload(UBlendSpaceGraph::StaticClass()->GetFName())
		, DocumentID(InDocumentID)
	{
	}

private:
	// The object that is the real payload
	TWeakObjectPtr<UBlendSpaceGraph> DocumentID;
};
