// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Delegates/Delegate.h"
#include "Engine/EngineBaseTypes.h"
#include "HAL/PlatformCrt.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

#include "AnimationBlueprintEditorSettings.generated.h"


// Settings for the Animation Blueprint Editor
UCLASS(config = EditorPerProjectUserSettings)
class UAnimationBlueprintEditorSettings : public UObject
{
	GENERATED_BODY()

public:
	/** If true, automatically pose watch selected nodes. */
	UPROPERTY(EditAnywhere, config, Category = "Debugging")
	bool bPoseWatchSelectedNodes = false;

	/** Whether to display the corner text in an animation graph. Changing this only affects newly opened graphs. */
	UPROPERTY(EditAnywhere, config, Category = "Graphs")
	bool bShowGraphCornerText = true;

public:
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnUpdateSettingsMulticaster, const UAnimationBlueprintEditorSettings*, EPropertyChangeType::Type);
	FOnUpdateSettingsMulticaster OnSettingsChange;

	FDelegateHandle RegisterOnUpdateSettings(const FOnUpdateSettingsMulticaster::FDelegate& Delegate)
	{
		return OnSettingsChange.Add(Delegate);
	}

	void UnregisterOnUpdateSettings(FDelegateHandle Object)
	{
		OnSettingsChange.Remove(Object);
	}

protected:
	//~ Begin UObject Interface
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject Interface
};
