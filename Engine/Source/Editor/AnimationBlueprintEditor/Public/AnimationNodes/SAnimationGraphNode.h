// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Engine/PoseWatch.h"
#include "HAL/Platform.h"
#include "Input/Reply.h"
#include "KismetNodes/SGraphNodeK2Base.h"
#include "Math/Vector2D.h"
#include "Misc/Attribute.h"
#include "SNodePanel.h"
#include "Styling/SlateColor.h"
#include "Templates/SharedPointer.h"
#include "Widgets/DeclarativeSyntaxSupport.h"

#define UE_API ANIMATIONBLUEPRINTEDITOR_API

class IDetailTreeNode;
class IPropertyRowGenerator;
class SGraphNode;
class SGraphPin;
class SNodeTitle;
class SPoseWatchOverlay;
class SVerticalBox;
class SWidget;
class UAnimBlueprint;
class UAnimGraphNode_Base;
class UEdGraphNode;
class UEdGraphPin;
struct FGeometry;
struct FGraphInformationPopupInfo;
struct FNodeInfoContext;
struct FOverlayWidgetInfo;
template <typename FuncType> class TFunctionRef;

class SAnimationGraphNode : public SGraphNodeK2Base
{
public:
	SLATE_BEGIN_ARGS(SAnimationGraphNode) {}
	SLATE_END_ARGS()

	// Reverse index of the error reporting bar slot
	static const int32 ErrorReportingSlotReverseIndex = 0;

	// Reverse index of the tag/functions slot
	static const int32 TagAndFunctionsSlotReverseIndex = 1;
	
	UE_API void Construct(const FArguments& InArgs, UAnimGraphNode_Base* InNode);

	// Tweak any created pin widgets so they respond to bindings
	static UE_API void ReconfigurePinWidgetsForPropertyBindings(UAnimGraphNode_Base* InAnimGraphNode, TSharedRef<SGraphNode> InGraphNodeWidget, TFunctionRef<TSharedPtr<SGraphPin>(UEdGraphPin*)> InFindWidgetForPin);

	// Create below-widget controls for editing anim node functions
	static UE_API TSharedRef<SWidget> CreateNodeFunctionsWidget(UAnimGraphNode_Base* InAnimNode, TAttribute<bool> InUseLowDetail);

	// Create below-widget controls for editing anim node tags
	static UE_API TSharedRef<SWidget> CreateNodeTagWidget(UAnimGraphNode_Base* InAnimNode, TAttribute<bool> InUseLowDetail);
	
protected:
	// SWidget interface
	UE_API virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	// End of SWidget interface

	// SGraphNode interface
	UE_API virtual TArray<FOverlayWidgetInfo> GetOverlayWidgets(bool bSelected, const FVector2f& WidgetSize) const override;
	UE_API virtual TSharedRef<SWidget> CreateTitleWidget(TSharedPtr<SNodeTitle> InNodeTitle) override;
	UE_API virtual void GetNodeInfoPopups(FNodeInfoContext* Context, TArray<FGraphInformationPopupInfo>& Popups) const override;
	UE_API virtual void CreateBelowPinControls(TSharedPtr<SVerticalBox> MainBox) override;
	UE_API virtual TSharedRef<SWidget> CreateNodeContentArea() override;
	virtual bool IsHidingPinWidgets() const override { return UseLowDetailNodeContent(); }
	// End of SGraphNode interface

private:
	// Handle the node informing us that the title has changed
	UE_API void HandleNodeTitleChanged();

	// LOD related functions for content area
	UE_API bool UseLowDetailNodeContent() const;
	UE_API FVector2D GetLowDetailDesiredSize() const;

	// Handler for pose watches changing
	UE_API void HandlePoseWatchesChanged(UAnimBlueprint* InAnimBlueprint, UEdGraphNode* InNode);

	/** Keep a reference to the indicator widget handing around */
	TSharedPtr<SWidget> IndicatorWidget;

	/** Keep a reference to the pose view indicator widget handing around */
	TSharedPtr<SPoseWatchOverlay> PoseViewWidget;

	/** Cache the node title so we can invalidate it */
	TSharedPtr<SNodeTitle> NodeTitle;

	/** Cached size from when we last drew at high detail */
	FVector2D LastHighDetailSize;

	/** Cached content area widget (used to derive LastHighDetailSize) */
	TSharedPtr<SWidget> CachedContentArea;
};

#undef UE_API
