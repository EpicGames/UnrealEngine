// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Layout/Visibility.h"
#include "Math/MathFwd.h"
#include "Math/Vector.h"
#include "PersonaDelegates.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class UAnimGraphNode_Base;
class UBlendSpace;

class SBlendSpacePreview : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SBlendSpacePreview){}

	SLATE_ARGUMENT(FOnGetBlendSpaceSampleName, OnGetBlendSpaceSampleName)

	SLATE_END_ARGS()

	ANIMATIONBLUEPRINTEDITOR_API void Construct(const FArguments& InArgs, UAnimGraphNode_Base* InNode);

protected:
	EVisibility GetBlendSpaceVisibility() const;
	bool GetBlendSpaceInfo(TWeakObjectPtr<const UBlendSpace>& OutBlendSpace, FVector& OutPosition, FVector& OutFilteredPosition) const;

	TWeakObjectPtr<const UAnimGraphNode_Base> Node;
	TWeakObjectPtr<const UBlendSpace> CachedBlendSpace;
	FVector CachedPosition;
	FVector CachedFilteredPosition;
};
