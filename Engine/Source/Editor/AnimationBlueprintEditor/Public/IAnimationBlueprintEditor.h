// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "BlueprintEditor.h"
#include "IHasPersonaToolkit.h"

class IAnimationSequenceBrowser;
class UAnimInstance;

class IAnimationBlueprintEditor : public FBlueprintEditor, public IHasPersonaToolkit
{
public:
	/** Get the last pin type we used to create a graph pin */
	virtual const FEdGraphPinType& GetLastGraphPinTypeUsed() const = 0;

	/** Set the last pin type we used to create a graph pin */
	virtual void SetLastGraphPinTypeUsed(const FEdGraphPinType& InType) = 0;

	/** Get the asset browser we host */
	virtual IAnimationSequenceBrowser* GetAssetBrowser() const = 0;

	/** Get the preview anim instance we are using, which can be a linked instance depending on preview settings */
	virtual UAnimInstance* GetPreviewInstance() const = 0;
	
	// Name of EditorToolkit, used for testing against FAssetEditorToolkit::GetToolkitFName 
	static FName GetAnimationBlueprintEditorToolkitName() { static FName ToolkitName("AnimationBlueprintEditor"); return ToolkitName; }
};
