// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/IToolkitHost.h"
#include "Modules/ModuleInterface.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxExtender.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "IAnimationBlueprintEditor.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAnimationBlueprintEditor, Log, All);

class FBlueprintActionDatabaseRegistrar;

class IAnimationBlueprintEditorModule : public IModuleInterface, public IHasMenuExtensibility, public IHasToolBarExtensibility
{
public:
	/**
	 * Creates an instance of an Animation Blueprint editor.
	 *
	 * Note: This function should not be called directly, use one of the following instead:
	 *	- FKismetEditorUtilities::BringKismetToFocusAttentionOnObject
	 *  - GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorForAsset
	 *
	 * @param	Mode					Mode that this editor should operate in
	 * @param	InitToolkitHost			When Mode is WorldCentric, this is the level editor instance to spawn this editor within
	 * @param	AnimBlueprint			The blueprint object to start editing.  If specified, Skeleton and AnimationAsset must be NULL.
	 *
	 * @return	Interface to the new Animation Blueprint editor
	 */
	virtual TSharedRef<IAnimationBlueprintEditor> CreateAnimationBlueprintEditor(const EToolkitMode::Type Mode, const TSharedPtr< class IToolkitHost >& InitToolkitHost, class UAnimBlueprint* Blueprint) = 0;


	/** Get all toolbar extenders */
	DECLARE_DELEGATE_RetVal_TwoParams(TSharedRef<FExtender>, FAnimationBlueprintEditorToolbarExtender, const TSharedRef<FUICommandList> /*InCommandList*/, TSharedRef<IAnimationBlueprintEditor> /*InAnimationBlueprintEditor*/);
	virtual TArray<FAnimationBlueprintEditorToolbarExtender>& GetAllAnimationBlueprintEditorToolbarExtenders() = 0;

	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGetCustomDebugObjects, const IAnimationBlueprintEditor& /*AnimationBlueprintEditor*/, TArray<FCustomDebugObject>& /*DebugList*/);
	virtual FOnGetCustomDebugObjects& OnGetCustomDebugObjects() = 0;

	/** Get anim blueprint type actions */
	virtual void GetTypeActions(FBlueprintActionDatabaseRegistrar& ActionRegistrar) const = 0;

	/** Get anim blueprint instance actions */
	virtual void GetInstanceActions(const UAnimBlueprint* InAnimBlueprint, FBlueprintActionDatabaseRegistrar& ActionRegistrar) const = 0;
};
